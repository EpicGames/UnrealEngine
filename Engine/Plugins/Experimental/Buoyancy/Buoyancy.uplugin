{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Buoyancy",
	"Description": "Set of tools for adding buoyant behavior to objects which interact with water bodies generated by the Water plugin",
	"Category": "Water",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "Buoyancy",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "Water",
			"Enabled": true
		},
		{
			"Name": "ChaosUserDataPT",
			"Enabled": true
		}
	]
}