// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Buoyancy : ModuleRules
{
	public Buoyancy(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Water",
				"ChaosUserDataPT",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ChaosCore",
				"Chaos",
				"PhysicsCore",
				"DeveloperSettings",
			}
		);
	}
}