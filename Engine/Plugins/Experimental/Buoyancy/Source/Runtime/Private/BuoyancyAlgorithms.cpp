// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyAlgorithms.h"
#include "BuoyancyStats.h"
#include "BuoyancyParticleData.h"
#include "DrawDebugHelpers.h"
#include "Engine/Engine.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Collision/CollisionFilter.h"
#include "Chaos/Collision/CollisionUtil.h"
#include "Chaos/Sphere.h"
#include "Chaos/ImplicitObjectScaled.h"

// Set to 1 to enable per-face/per-triangle SCOPE_CYCLE_COUNTERs in the inner loops.
// These are useful for profiling but add measurable overhead at ~1M+ iterations/tick.
#ifndef BUOYANCY_INNER_LOOP_STATS
#define BUOYANCY_INNER_LOOP_STATS 0
#endif

//
// CVars
//

extern bool bBuoyancyDebugDraw;

bool bBuoyancyAlgorithmsAllowVolRatioOverOne = false;
FAutoConsoleVariableRef CVarBuoyancyAlgorithmsAllowVolRatioOverOne(TEXT("p.Buoyancy.Algorithms.AllowVolRatioOverOne"), bBuoyancyAlgorithmsAllowVolRatioOverOne, TEXT(""));

bool bBuoyancyAlgorithmsAccurateBuoyancyAlongNormal = false;
FAutoConsoleVariableRef CVarBuoyancyAlgorithmsAccurateBuoyancyAlongNormal(TEXT("p.Buoyancy.Algorithms.AccurateBuoyancyAlongNormal"), bBuoyancyAlgorithmsAccurateBuoyancyAlongNormal, TEXT(""));

bool bBuoyancyAlgorithmsUseConvexAsBox = false;
FAutoConsoleVariableRef CVarBuoyancyAlgorithmsUseConvexAsBox(TEXT("p.Buoyancy.Algorithms.UseConvexAsBox"), bBuoyancyAlgorithmsUseConvexAsBox, TEXT(""));

//
// Internal Functions
//

namespace
{
	using Chaos::FVec3;
	using Chaos::FAABB3;

	// Check to see if an object's shape is marked as already submerged
	bool IsShapeSubmerged_Internal(const TSparseArray<TBitArray<>>& SubmergedShapes, const int32 ParticleIndex, const int32 ShapeIndex)
	{
		return
			SubmergedShapes.IsValidIndex(ParticleIndex) &&
			SubmergedShapes[ParticleIndex].IsValidIndex(ShapeIndex) &&
			SubmergedShapes[ParticleIndex][ShapeIndex];
	}

	// Mark an object's shape as submerged
	void SubmergeShape_Internal(TSparseArray<TBitArray<>>& SubmergedShapes, const int32 ParticleIndex, const int32 ShapeIndex)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_SubmergeShapeInternal)

		// If no shapes are tracked for this particle yet, add a bit array for it
		if (SubmergedShapes.IsValidIndex(ParticleIndex) == false)
		{
			SubmergedShapes.Insert(ParticleIndex, TBitArray<>(false, ShapeIndex + 1));
		}

		// If the bit array for this particle existed already but is too small, expand it
		else if (SubmergedShapes[ParticleIndex].IsValidIndex(ShapeIndex) == false)
		{
			SubmergedShapes[ParticleIndex].SetNum(ShapeIndex + 1, false);
		}

		// Mark this particle's shape as submerged
		SubmergedShapes[ParticleIndex][ShapeIndex] = true;
	}

	// NOTE: See SubdivideBounds(...) for a description of this algorithm
	//
	// TODO: Use an array + offset rather than raw ptr
	// TODO: Use a bounds stack instead of a recursive algo
	// TODO: Only build overlapping parts of hierarchy
	void SubdivideBounds_Internal(const FAABB3& Bounds, int32 NumSubdivisions, FAABB3** BoundsPtrPtr)
	{
		// We assume that the buffer pointed to by *BoundsPtr 
		const FVec3 Min = Bounds.Min();
		const FVec3 Max = Bounds.Max();
		const FVec3 Cen = Bounds.GetCenter();

		// Decrement subdivisions and track whether we have any more to go
		const bool bSubdivide = --NumSubdivisions > 0;

		// Get a pointer to the first writable element in the array of bounds,
		// or to a temporary swap space of bounds if we haven't reached the
		// leaf level yet
		FAABB3 BoundsSwap[8];
		FAABB3* BoundsPtr = bSubdivide ? BoundsSwap : *BoundsPtrPtr;

		// Generate 8 subdivisions
		BoundsPtr[0] = FAABB3(Min, Cen);
		BoundsPtr[1] = FAABB3(FVec3(Cen.X, Min.Y, Min.Z), FVec3(Max.X, Cen.Y, Cen.Z));
		BoundsPtr[2] = FAABB3(FVec3(Min.X, Cen.Y, Min.Z), FVec3(Cen.X, Max.Y, Cen.Z));
		BoundsPtr[3] = FAABB3(FVec3(Min.X, Min.Y, Cen.Z), FVec3(Cen.X, Cen.Y, Max.Z));
		BoundsPtr[4] = FAABB3(Cen, Max);
		BoundsPtr[5] = FAABB3(FVec3(Min.X, Cen.Y, Cen.Z), FVec3(Cen.X, Max.Y, Max.Z));
		BoundsPtr[6] = FAABB3(FVec3(Cen.X, Min.Y, Cen.Z), FVec3(Max.X, Cen.Y, Max.Z));
		BoundsPtr[7] = FAABB3(FVec3(Cen.X, Cen.Y, Min.Z), FVec3(Max.X, Max.Y, Cen.Z));

		if (bSubdivide)
		{
			// Recurse if we haven't reached the leaf level yet
			for (int32 Index = 0; Index < 8; ++Index)
			{
				SubdivideBounds_Internal(BoundsPtr[Index], NumSubdivisions, BoundsPtrPtr);
			}
		}
		else
		{
			// Increment the bounds ptr by 8 if we just wrote 8 leaves
			(*BoundsPtrPtr) += 8;
		}
	}
}


//
// Algorithms
//

namespace BuoyancyAlgorithms
{
	using Chaos::FVec3;
	using Chaos::FAABB3;
	using Chaos::FAABBEdge;
	using Chaos::FRealSingle;
	using Chaos::FPBDRigidsEvolutionGBF;
	using Chaos::FGeometryParticleHandle;
	using Chaos::FPBDRigidParticleHandle;
	using Chaos::FImplicitObject;
	using Chaos::FRigidTransform3;
	using Chaos::FMatrix33;
	using Chaos::FChaosPhysicsMaterial;
	using Chaos::FShapeInstance;
	using Chaos::FShapeInstanceArray;
	using Chaos::EImplicitObjectType;
	using Chaos::FConstGenericParticleHandle;
	using Chaos::FImplicitSphere3;
	using Chaos::FConvex;
	using Chaos::TImplicitObjectScaled;
	using Chaos::TImplicitObjectInstanced;

	FRealSingle ComputeParticleVolume(const FPBDRigidsEvolutionGBF& Evolution, const FGeometryParticleHandle* Particle)
	{
		const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle();
		if (Rigid == nullptr)
		{
			return -1.f;
		}

		const FChaosPhysicsMaterial* ParticleMaterial = Evolution.GetFirstClusteredPhysicsMaterial(Particle);
		if (ParticleMaterial == nullptr)
		{
			return -1.f;
		}

		// Get the material density from the submerged particle's material and use that
		// in conjunction with its mass to compute its effective total volume.
		//
		// Use this as the upper bound for submerged volume, since the voxelized
		// submerged shape bounds will likely have overestimated the "true" volume
		// of the object.
		//
		// NOTE: This is using the density of the material of the FIRST shape on the
		// object, whatever it is. If for example the particle is a cluster union of
		// GCs of totally different types, this might be an incorrect volume.
		//
		// However, the volumes or masses of each "true" shape are not accessible to
		// us, so at the moment this is nearly the best estimate we'll be able to get.
		const FRealSingle ParticleDensity = Chaos::GCm3ToKgCm3(ParticleMaterial->Density);
		const FRealSingle ParticleMass = Rigid->M();
		const FRealSingle ParticleVol
			= ParticleDensity > UE_SMALL_NUMBER
			? ParticleMass / ParticleDensity
			: 0.f;
		return ParticleVol;
	}

	FRealSingle ComputeShapeVolume(const FGeometryParticleHandle* Particle, const bool UseBoundingBoxVolume)
	{
		if (Particle == nullptr)
		{
			return -1.f;
		}

		const FImplicitObject* ImplicitObject = Particle->GetGeometry();
		if (ImplicitObject == nullptr)
		{
			return -1.f;
		}

		const FShapeInstanceArray& ShapeInstances = Particle->ShapeInstances();
		if (ShapeInstances.Num() == 0)
		{
			return -1.f;
		}

		// Loop over every leaf object and sum up the volume of each of their bounds
		// to get an upper limit on the submerged volume that can be reported by
		// ComputeSubmergedVolume.
		FRealSingle ShapeVol = 0.f;
		ImplicitObject->VisitLeafObjects(
			[Particle, UseBoundingBoxVolume , &ShapeInstances, &ShapeVol]
			(const FImplicitObject* InnerImplicitObject, const FRigidTransform3&, const int32 RootObjectIndex, const int32, const int32)
		{
			const int32 ShapeIndex = ShapeInstances.IsValidIndex(RootObjectIndex) ? RootObjectIndex : 0;
			const EImplicitObjectType ShapeType = Chaos::Private::GetImplicitCollisionType(Particle, InnerImplicitObject);
			if (DoCollide(ShapeType, ShapeInstances[ShapeIndex].Get()))
			{
				Chaos::Utilities::CastHelper(*InnerImplicitObject, [UseBoundingBoxVolume , &ShapeVol](const auto& Geom)
				{
					if (UseBoundingBoxVolume)
					{
						ShapeVol += Geom.BoundingBox().GetVolume();
					}
					else
					{
						ShapeVol += Geom.GetVolume();
					}
				});
			}
		});

		/*
		NOTE:	This version is more performant, but visits invalid leaf objects and crashes
				because of the utility cast function.
		
		FRealSingle ShapeVol = 0.f;
		Utilities::VisitConcreteObjects(*ImplicitObject,
		[Particle, &ShapeInstances, &ShapeVol](const auto& Geom, int32 ShapeIndex)
		{
			const EImplicitObjectType ShapeType = Private::GetImplicitCollisionType(Particle, &Geom);
			if (DoCollide(ShapeType, ShapeInstances[ShapeIndex].Get()))
			{
				ShapeVol += Geom.BoundingBox().GetVolume();
			}
		});
		*/

		return ShapeVol;
	}

	void ComputeShapeVolumes(const FGeometryParticleHandle* Particle, FRealSingle& OutBBoxVol, FRealSingle& OutGeomVol)
	{
		OutBBoxVol = 0.f;
		OutGeomVol = 0.f;

		if (Particle == nullptr)
		{
			return;
		}

		const FImplicitObject* ImplicitObject = Particle->GetGeometry();
		if (ImplicitObject == nullptr)
		{
			return;
		}

		const FShapeInstanceArray& ShapeInstances = Particle->ShapeInstances();
		if (ShapeInstances.Num() == 0)
		{
			return;
		}

		ImplicitObject->VisitLeafObjects(
			[Particle, &ShapeInstances, &OutBBoxVol, &OutGeomVol]
			(const FImplicitObject* InnerImplicitObject, const FRigidTransform3&, const int32 RootObjectIndex, const int32, const int32)
		{
			const int32 ShapeIndex = ShapeInstances.IsValidIndex(RootObjectIndex) ? RootObjectIndex : 0;
			const EImplicitObjectType ShapeType = Chaos::Private::GetImplicitCollisionType(Particle, InnerImplicitObject);
			if (DoCollide(ShapeType, ShapeInstances[ShapeIndex].Get()))
			{
				Chaos::Utilities::CastHelper(*InnerImplicitObject, [&OutBBoxVol, &OutGeomVol](const auto& Geom)
				{
					const FRealSingle BBoxVol = Geom.BoundingBox().GetVolume();
					const FRealSingle GeomVol = Geom.GetVolume();
					OutBBoxVol += BBoxVol;
					// Some implicit types return 0 from GetVolume(); fall back to BBox volume
					OutGeomVol += (GeomVol > UE_SMALL_NUMBER) ? GeomVol : BBoxVol;
				});
			}
		});
	}

	void ScaleSubmergedVolume(const FPBDRigidsEvolutionGBF& Evolution, const FGeometryParticleHandle* Particle, const bool UseBoundingBoxVolume, FRealSingle& SubmergedVol, FRealSingle& TotalVol)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ScaleSubmergedVolume)

		// Get submerged object's "particle" volume and "shape" volume.
		//
		// The particle volume is the theoretical volume of the particle,
		// derived from its mass and density.
		//
		// The shape volume is the volume of all shape bounds which can
		// possibly count as submerged volumes.
		const FRealSingle ParticleVol = ComputeParticleVolume(Evolution, Particle);
		const FRealSingle ShapeVol = ComputeShapeVolume(Particle, UseBoundingBoxVolume);
		TotalVol = ParticleVol;

		// If the submerged vol somehow exceeded the max shape vol, clamp it
		if (SubmergedVol - ShapeVol > UE_SMALL_NUMBER)
		{
			SubmergedVol = ShapeVol;
		}

		// Adjust the output volume based on the ratio of the material volume and the shape volume.
		//
		// In most cases, we expect the shape volume to have overestimated the submerged volume for
		// most shapes, especially those which are hollow.
		//
		// In some cases, if the mass of the submerged object has been changed independently of the
		// density, which increases
		if (ParticleVol > UE_SMALL_NUMBER &&
			ShapeVol > UE_SMALL_NUMBER &&
			(bBuoyancyAlgorithmsAllowVolRatioOverOne || ParticleVol < ShapeVol))
		{
			const float VolRatio = ParticleVol / ShapeVol;
			SubmergedVol *= VolRatio;
		}
	}

	// ScaleSubmergedVolume overload using precomputed particle/shape volumes
	// to avoid repeated VisitLeafObjects traversals per shape and per particle.
	void ScaleSubmergedVolume(const FRealSingle ParticleVol, const FRealSingle ShapeVol,
		FRealSingle& SubmergedVol, FRealSingle& TotalVol)
	{
		TotalVol = ParticleVol;

		if (SubmergedVol - ShapeVol > UE_SMALL_NUMBER)
		{
			SubmergedVol = ShapeVol;
		}

		if (ParticleVol > UE_SMALL_NUMBER &&
			ShapeVol > UE_SMALL_NUMBER &&
			(bBuoyancyAlgorithmsAllowVolRatioOverOne || ParticleVol < ShapeVol))
		{
			const float VolRatio = ParticleVol / ShapeVol;
			SubmergedVol *= VolRatio;
		}
	}


	bool ComputeSubmergedVolume(FBuoyancyParticleData& ParticleData, const FPBDRigidsEvolutionGBF& Evolution, const FGeometryParticleHandle* SubmergedParticle, const FGeometryParticleHandle* WaterParticle, const FVector& WaterX, const FVector& WaterN, int32 NumSubdivisions, float MinVolume, float& SubmergedVol, FVec3& SubmergedCoM, float& TotalVol)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeSubmergedVolume)

		if (ComputeSubmergedVolume(ParticleData, SubmergedParticle, WaterParticle, WaterX, WaterN, NumSubdivisions, MinVolume, SubmergedVol, SubmergedCoM))
		{
			ScaleSubmergedVolume(Evolution, SubmergedParticle, true, SubmergedVol, TotalVol);


#if CHAOS_DEBUG_DRAW
			if (bBuoyancyDebugDraw)
			{
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(SubmergedCoM, FColor::Yellow, false, -1.f, -1, 15.f);
			}
#endif


			return true;
		}

		return false;
	}


	bool ComputeSubmergedVolume(FBuoyancyParticleData& ParticleData, const FGeometryParticleHandle* SubmergedParticle, const FGeometryParticleHandle* WaterParticle, const FVector& WaterX, const FVector& WaterN, int32 NumSubdivisions, float MinVolume, float& SubmergedVol, FVec3& SubmergedCoM)
	{
		// Get some initial data about the submerged particle
		const FImplicitObject* RootImplicit = SubmergedParticle->GetGeometry();
		const FShapeInstanceArray& ShapeInstances = SubmergedParticle->ShapeInstances();
		const FConstGenericParticleHandle SubmergedGeneric = SubmergedParticle;
		const FRigidTransform3 ParticleWorldTransform = SubmergedGeneric->GetTransformPQ();
		const int32 ParticleIndex = ParticleData.GetIndex(*SubmergedParticle);
		TSparseArray<TBitArray<>>& SubmergedShapes = ParticleData.SubmergedShapes;

		// Some info about the water
		const FImplicitObject* WaterRootImplicit = WaterParticle->GetGeometry();
		const EImplicitObjectType WaterShapeType = Chaos::Private::GetImplicitCollisionType(WaterParticle, WaterRootImplicit);
		const FShapeInstanceArray& WaterShapeInstances = WaterParticle->ShapeInstances();
		const FShapeInstance* WaterShapeInstance = WaterShapeInstances[0].Get();

		// Initialize submersion values
		SubmergedVol = 0.f;
		SubmergedCoM = FVec3::ZeroVector;

		// Traverse the submerged particle's leaves
		RootImplicit->VisitLeafObjects(
			[SubmergedParticle, ParticleIndex, &ShapeInstances, WaterShapeType, WaterShapeInstance, &ParticleWorldTransform, &WaterX, &WaterN, &NumSubdivisions, &MinVolume, &SubmergedShapes, &SubmergedVol, &SubmergedCoM]
			(const FImplicitObject* Implicit, const FRigidTransform3& RelativeTransform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)
		{
			const FAABB3 RelativeBounds = Implicit->CalculateTransformedBounds(RelativeTransform);
			const int32 ShapeIndex = (ShapeInstances.IsValidIndex(RootObjectIndex)) ? RootObjectIndex : 0;
			const FShapeInstance* ShapeInstance = ShapeInstances[ShapeIndex].Get();
			const EImplicitObjectType ShapeType = Chaos::Private::GetImplicitCollisionType(SubmergedParticle, Implicit);

			// If this shape pair doesn't pass a narrow phase test then skip it
			if (!ShapePairNarrowPhaseFilter(ShapeType, ShapeInstance, WaterShapeType, WaterShapeInstance))
			{
				return;
			}

			// If this shape has already been submerged, skip it to avoid double-counting
			// any buoyancy contributions.
			if (IsShapeSubmerged_Internal(SubmergedShapes, ParticleIndex, ObjectIndex))
			{
				return;
			}

			// Get the world-space bounds of shape A
			FRigidTransform3 ShapeWorldTransform = RelativeTransform * ParticleWorldTransform;
			FAABB3 LocalBox = Implicit->BoundingBox();
			if (const FImplicitSphere3* Sphere = Implicit->AsA<FImplicitSphere3>())
			{
				// If we have a sphere, ignore rotation because submerged volume is independent of rotation
				// and also we don't want to apply any torques on the wheel.
				// @todo(chaos): ComputeSubmergedBounds special case for spheres
				const FVec3 SphereCenter = ShapeWorldTransform.TransformPosition(FVec3(Sphere->GetCenterf()));
				const FVec3 SphereExtent = FVec3(Sphere->GetRadiusf());
				LocalBox = FAABB3(-SphereExtent, SphereExtent);
				ShapeWorldTransform.SetTranslation(SphereCenter);
				ShapeWorldTransform.SetRotation(FRotationMatrix::MakeFromZ(WaterN).ToQuat());
			}
			const FAABB3 WorldBox = LocalBox.TransformedAABB(ShapeWorldTransform);

#if CHAOS_DEBUG_DRAW
			//if (bBuoyancyDebugDraw)
			//{
			//	Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(
			//		ShapeWorldTransform.TransformPosition(LocalBox.GetCenter()),
			//		LocalBox.Extents() * .5f,
			//		ShapeWorldTransform.GetRotation(),
			//		FColor::Green, false, -1.f, SDPG_Foreground, 1.f);
			//}
#endif

			// Get the world space position of the shape
			const FVec3 ShapePos = ShapeWorldTransform.GetTranslation();

			// Get the projection of the shape position onto the water
			const FVec3 ShapeDiff = ShapePos - WaterX;
			const FVec3 ShapeSurfacePos = WaterX + ShapeDiff - (WaterN * FVec3::DotProduct(WaterN, ShapeDiff));

			// Get the position and normal on the surface relative to the box
			const FVec3 ShapeSurfacePosLocal = ShapeWorldTransform.InverseTransformPosition(ShapeSurfacePos);
			const FVec3 SurfaceNormalLocal = ShapeWorldTransform.InverseTransformVector(WaterN);

			// Generate subdivided bounds list
			TArray<FAABB3> SubmergedBoxes;
			SubdivideBounds(LocalBox, NumSubdivisions, MinVolume, SubmergedBoxes);

			// Loop over every subdivision of the shape bounds, counting up submerged portions
			bool bSubmerged = false;
			for (const FAABB3& Box : SubmergedBoxes)
			{
				// Compute the portion of the object bounds that are submerged
				FAABB3 SubmergedBox;
				if (ComputeSubmergedBounds(ShapeSurfacePosLocal, SurfaceNormalLocal, Box, SubmergedBox))
				{
					// At this point we know that the shape is submerged
					bSubmerged = true;

					// This bounds box is submerged. Compute it's volume and center of mass
					// in world space, and add those contributions to the submerged quantity.
					const FVec3 LeafSubmergedCoM = ShapeWorldTransform.TransformPosition(SubmergedBox.GetCenter());
					const float LeafSubmergedVol = SubmergedBox.GetVolume();
					SubmergedVol += LeafSubmergedVol;
					SubmergedCoM += LeafSubmergedCoM * LeafSubmergedVol;

					// Make sure the volume of the submerged portion never exceeds the total
					// volume of the leaf bounds
					const float LeafMaxVol = LocalBox.GetVolume() + UE_SMALL_NUMBER;
					ensureAlwaysMsgf(LeafSubmergedVol <= LeafMaxVol, TEXT("BuoyancyAlgorithms::ComputeSubmergedVolume: The volume of the submerged portion of the leaf bounds has somehow exceeded the volume of the overall leaf bounds."));



#if CHAOS_DEBUG_DRAW
					if (bBuoyancyDebugDraw)
					{
						//Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(
//							ShapeWorldTransform.TransformPosition(SubmergedBox.GetCenter()),
							//SubmergedBox.Extents() * .5f,
							//ShapeWorldTransform.GetRotation(),
							//FColor::Red, false, -1.f, SDPG_Foreground, 1.f);
					}
#endif
				}
			}

			if (bSubmerged)
			{
				SubmergeShape_Internal(SubmergedShapes, ParticleIndex, ObjectIndex);
			}
		});

		if (SubmergedVol > SMALL_NUMBER)
		{
			SubmergedCoM /= SubmergedVol;
			return true;
		}

		return false;
	}

	bool ComputeSubmergedBounds(const FVector& SurfacePoint, const FVector& SurfaceNormal, const FAABB3& RigidBox, FAABB3& OutSubmergedBounds)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeSubmergedBounds)

		// Partly submerged object can have at most 10 points intersecting
		// with the water surface
		static constexpr uint8 SubmergedVerticesMax = 10;
		FVec3 SubmergedVertices[SubmergedVerticesMax];
		uint8 SubmergedVerticesNum = 0;

		// Find bound box indices that are submerged, and build an array
		// of box vertices
		FVec3 Vertices[8];
		for (int32 VertexIndex = 0; VertexIndex < 8; ++VertexIndex)
		{
			FVec3& Vertex = Vertices[VertexIndex];
			Vertex = RigidBox.GetVertex(VertexIndex);
			const float Depth = SurfaceNormal.Dot(SurfacePoint - Vertex);
			if (Depth > SMALL_NUMBER)
			{
				SubmergedVertices[SubmergedVerticesNum++] = Vertex;
			}
		}

		// If no box corners were submerged, then there can be no submerged edges so stop here
		if (SubmergedVerticesNum == 0)
		{
			return false;
		}

		// If all box corners were submerged, then return the original box
		if (SubmergedVerticesNum == 8)
		{
			OutSubmergedBounds = RigidBox;
			return true;
		}

		// Find intersections of AABB edges with the surface and add these
		// points to the submerged verts list
		for (int32 EdgeIndex = 0; EdgeIndex < 12; ++EdgeIndex)
		{
			const FAABBEdge Edge = RigidBox.GetEdge(EdgeIndex);
			const FVec3 Vert0 = Vertices[Edge.VertexIndex0];
			const FVec3 Vert1 = Vertices[Edge.VertexIndex1];
			const float Depth0 = SurfaceNormal.Dot(SurfacePoint - Vert0);
			const float Depth1 = SurfaceNormal.Dot(SurfacePoint - Vert1);
			const bool bSubmerged0 = (Depth0 > SMALL_NUMBER);
			const bool bSubmerged1 = (Depth1 > SMALL_NUMBER);
			if (bSubmerged0 ^ bSubmerged1)
			{
				const float DepthDiff = Depth0 - Depth1;
				const float DepthAlpha = Depth0 / DepthDiff; // NOTE: Since one is submerged and one is not, we know that |DepthDiff| > 0
				const FVec3 SurfaceVertex = FMath::Lerp(Vert0, Vert1, DepthAlpha);
				SubmergedVertices[SubmergedVerticesNum++] = SurfaceVertex;

				// No point in continuing if we've filled our cache - we know
				// already that the remaining edges will be fruitless
				if (SubmergedVerticesNum == SubmergedVerticesMax)
				{
					break;
				}
			}
		}

		// Build and return an AABB which contains the submerged vertices of the rigid bounds
		OutSubmergedBounds = FAABB3(SubmergedVertices[0], SubmergedVertices[0]);
		for (uint8 VertexIndex = 1; VertexIndex < SubmergedVerticesNum; ++VertexIndex)
		{
			OutSubmergedBounds.GrowToInclude(SubmergedVertices[VertexIndex]);
		}
		return true;
	}


	bool SubdivideBounds(const FAABB3& Bounds, int32 NumSubdivisions, const float MinVolume, TArray<FAABB3>& OutBounds)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_SubdivideBounds)

		// Initialize bounds to an array of just the original bounds;
		OutBounds = TArray<FAABB3>{ Bounds };

		// If the bounds volume is already too small to subdivide, return the original bounds only
		const float Volume = Bounds.GetVolume();
		if (Volume < SMALL_NUMBER || Volume <= MinVolume)
		{
			return false;
		}

		// If V_0 is the volume of the outermost AABB, then the volume of
		// one box in the n'th subdivision of an AABB is given by
		// 
		// V_n = V_0 * 2^(-3 n)
		//
		// We can invert this equation to find the level of subdivisions
		// at which the volume becomes smaller than V_min.
		//
		// n < -(1/3) * log2(V_min / V_0)
		const int32 MaxNumSubdivisions = int32(-(1.f/3.f) * FMath::Log2(MinVolume / Volume));
		NumSubdivisions = FMath::Min(MaxNumSubdivisions, NumSubdivisions);

		// If we have any subdivisions to process, do them now
		if (NumSubdivisions > 0)
		{
			// Predetermine the total number of boxes we're going to generate, and allocate them
			// in a block.
			const int32 NumBounds = (int32)FMath::Pow(8.f, NumSubdivisions);
			OutBounds.SetNum(NumBounds);
			FAABB3* BoundsPtr = OutBounds.GetData();

			// Recursively generate boxes
			SubdivideBounds_Internal(Bounds, NumSubdivisions, &BoundsPtr);

			// Make sure that we didn't write too many or too few boxes
			const FAABB3* BoundsPtrMax = OutBounds.GetData() + NumBounds;
			ensureAlways(BoundsPtr == BoundsPtrMax);

			// Return the box array
			return true;
		}

		return false;
	}

	bool ComputeBuoyantForce(const FPBDRigidParticleHandle* RigidParticle, const float DeltaSeconds, const float WaterDensity, const float WaterDrag, const FVec3& GravityAccelVec, const FVec3& SubmergedCoM, const float SubmergedVol, const FVec3& WaterVel, const FVec3& WaterN, FVec3& OutDeltaV, FVec3& OutDeltaW)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeBuoyantForces)

		// NOTE: We assume gravity is -Z for perf... If we want to support buoyancy for
		// weird gravity setups, this is where we'd have to fix it up.
		const FVec3 GravityDir = FVec3::DownVector;
		const float GravityAccel = FVec3::DotProduct(GravityDir, GravityAccelVec);

		// Compute buoyant force
		//
		// NOTE: This is easy to compute with Archimedes' principle
		// https://en.wikipedia.org/wiki/Buoyancy
		//
		const float BuoyantForce = WaterDensity * SubmergedVol * GravityAccel;
		//                       = [ kg / cm^3 ] * [ cm^3 ]    * [cm / s^2]
		//                       = [ kg * cm / s^2 ]
		//                       = [ force ]

		// Only proceed if buoyant force isn't vanishingly small
		if (BuoyantForce < SMALL_NUMBER)
		{
			return false;
		}

		// Get a generic particle wrapper
		FConstGenericParticleHandle RigidGeneric(RigidParticle);

		// Get inverse inertia data to compute world space accelerations
		const FVec3 WorldCoM = RigidGeneric->PCom();
		const FVec3 CoMDiff = SubmergedCoM - WorldCoM;
		const FMatrix33 WorldInvI = Chaos::Utilities::ComputeWorldSpaceInertia(RigidGeneric->RCom(), RigidGeneric->ConditionedInvI());

		// Compute world buoyant force and torque
		FVec3 WorldForce;
		WorldForce = WaterN * BuoyantForce;

		const FVec3 WorldTorque = Chaos::FVec3::CrossProduct(CoMDiff, WorldForce);

		// Use inertia to convert forces to accelerations
		const FVec3 LinearAccel = RigidGeneric->InvM() * WorldForce;
		const FVec3 AngularAccel = WorldInvI * WorldTorque;

		// Integrate to get delta velocities
		OutDeltaV = LinearAccel * DeltaSeconds;
		OutDeltaW = AngularAccel * DeltaSeconds;

		// Get the velocities of the submerged portion relative to the water - We want the
		// drag force to bring these values to zero
		const FVec3 SubmergedV = RigidParticle->GetV() + FVec3::CrossProduct(RigidParticle->GetW(), CoMDiff);
		const FVec3 RelativeV = SubmergedV - WaterVel;

		// Compute water drag force
		//
		// NOTE: This is a very approximate "ether drag" style model here, probably 
		// we should scale the model with submerged volume for more accuracy,WorldBox
		// and apply the drag force in opposition to the linear motion of the submerged
		// center of mass.
		const float DragFactor = FMath::Max(0.f, 1.f - (WaterDrag * DeltaSeconds));

		// Account for water drag in deltas
		OutDeltaV = (DragFactor * OutDeltaV) + (DragFactor - 1.f) * RelativeV;
		OutDeltaW = (DragFactor * OutDeltaW) + (DragFactor - 1.f) * RigidParticle->GetW();

		//
		return true;
	}

	template <typename SamplerType>
	void ComputeSubmergedVolumeAndForcesForParticle(FBuoyancyParticleData& ParticleData,
		const Chaos::FGeometryParticleHandle* SubmergedParticle, const FGeometryParticleHandle* WaterParticle,
		TSharedPtr<SamplerType> WaterSampler,
		const Chaos::FPBDRigidsEvolution& Evolution, const float DeltaSeconds, const float WaterDensity, const float WaterDrag, const float WaterLift,
		float &OutTotalParticleVol, float &OutTotalSubmergedVol, Chaos::FVec3 &OutTotalSubmergedCoM,
		Chaos::FVec3 &OutTotalDragForce, Chaos::FVec3 &OutTotalDragTorque,
		Chaos::FVec3 &OutTotalBuoyancyForce, Chaos::FVec3 &OutTotalBuoyancyTorque)
	{		
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForParticle)

		// Initialize cumulative values
		OutTotalParticleVol = 0.f;
		OutTotalSubmergedVol = 0.f;
		OutTotalSubmergedCoM = FVec3::ZeroVector;
		OutTotalDragForce = FVec3::ZeroVector;
		OutTotalDragTorque = FVec3::ZeroVector;
		OutTotalBuoyancyForce = FVec3::ZeroVector;
		OutTotalBuoyancyTorque = FVec3::ZeroVector;

		// Get some initial data about the submerged particle
		const FImplicitObject* RootImplicit = SubmergedParticle->GetGeometry();
		const FShapeInstanceArray& ShapeInstances = SubmergedParticle->ShapeInstances();
		const FConstGenericParticleHandle SubmergedGeneric = SubmergedParticle;
		const FRigidTransform3 ParticleWorldTransform = SubmergedGeneric->GetTransformPQ();
		const int32 ParticleIndex = ParticleData.GetIndex(*SubmergedParticle);
		TSparseArray<TBitArray<>>& SubmergedShapes = ParticleData.SubmergedShapes;

		// Some info about the water
		const FImplicitObject* WaterRootImplicit = WaterParticle->GetGeometry();
		const EImplicitObjectType WaterShapeType = Chaos::Private::GetImplicitCollisionType(WaterParticle, WaterRootImplicit);
		const FShapeInstanceArray& WaterShapeInstances = WaterParticle->ShapeInstances();
		const FShapeInstance* WaterShapeInstance = WaterShapeInstances[0].Get();

		WaterSampler->InitializeForSubmersion();

		// Precompute particle and shape volumes — both are constant per particle per tick
		// and would otherwise be recomputed via VisitLeafObjects on every shape + once per particle.
		const FRealSingle ParticleVol = ComputeParticleVolume(Evolution, SubmergedParticle);
		FRealSingle ShapeVolBBox, ShapeVolGeom;
		ComputeShapeVolumes(SubmergedParticle, ShapeVolBBox, ShapeVolGeom);

		// Traverse the submerged particle's leaves
		RootImplicit->VisitLeafObjects(
			[&Evolution, DeltaSeconds, WaterDensity, WaterDrag, WaterLift, SubmergedParticle, &WaterSampler,
			ParticleVol, ShapeVolBBox, ShapeVolGeom,
			ParticleIndex, &ShapeInstances, WaterShapeType, WaterShapeInstance, &ParticleWorldTransform,
			&SubmergedShapes, &OutTotalSubmergedVol, &OutTotalSubmergedCoM,
			&OutTotalDragForce, &OutTotalDragTorque, &OutTotalBuoyancyForce, &OutTotalBuoyancyTorque]
			(const FImplicitObject* Implicit, const FRigidTransform3& RelativeTransform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)
			{								
				const int32 ShapeIndex = (ShapeInstances.IsValidIndex(RootObjectIndex)) ? RootObjectIndex : 0;
				const FShapeInstance* ShapeInstance = ShapeInstances[ShapeIndex].Get();
				const EImplicitObjectType ShapeType = Chaos::Private::GetImplicitCollisionType(SubmergedParticle, Implicit);

				// If this shape pair doesn't pass a narrow phase test then skip it
				if (!ShapePairNarrowPhaseFilter(ShapeType, ShapeInstance, WaterShapeType, WaterShapeInstance))
				{
					return;
				}

				// If this shape has already been submerged, skip it to avoid double-counting
				// any buoyancy contributions.
				if (IsShapeSubmerged_Internal(SubmergedShapes, ParticleIndex, ObjectIndex))
				{
					return;
				}
				
				float OutSubmergedVol;
				FVec3 OutSubmergedCoM;
				FVec3 OutForce, OutTorque;
				FVec3 OutBuoyancyForce, OutBuoyancyTorque;				

				// Get the world-space bounds of shape A
				FRigidTransform3 ShapeWorldTransform = RelativeTransform * ParticleWorldTransform;
				FRigidTransform3 ShapeWorldTransformScaled = ShapeWorldTransform;

				const FConvex* Convex = nullptr;
				
				FAABB3 LocalBox = Implicit->BoundingBox();

				// Get the world-space bounds of shape A					
				if (const FImplicitSphere3* Sphere = Implicit->AsA<FImplicitSphere3>())
				{
					// If we have a sphere, ignore rotation because submerged volume is independent of rotation
					// and also we don't want to apply any torques on the wheel.			
					const FVec3 SphereCenter = ShapeWorldTransform.TransformPosition(FVec3(Sphere->GetCenterf()));
					const FVec3 SphereExtent = FVec3(Sphere->GetRadiusf());
										
					LocalBox = FAABB3(-SphereExtent, SphereExtent);
					ShapeWorldTransform.SetTranslation(SphereCenter);

					// #todo(dmp): at this point, we haven't figured out the water normal so we use up vector for now
					FVec3 TmpWaterN = { 0.f, 0.f, 1.f };
					ShapeWorldTransform.SetRotation(FRotationMatrix::MakeFromZ(TmpWaterN).ToQuat());
				}
				else if (const FConvex* TheConvex = Implicit->AsA<FConvex>())
				{
					Convex = TheConvex;
				}
				else if (const TImplicitObjectScaled<FConvex>* ScaledObject = TImplicitObjectScaled<FConvex>::AsScaled(*Implicit))
				{
					Convex = ScaledObject->GetUnscaledObject();

					if (Convex != nullptr)
					{
						FRigidTransform3 RelativeTransformTmp = RelativeTransform;
						RelativeTransformTmp.SetScale3D(RelativeTransform.GetScale3D() * ScaledObject->GetScale());

						ShapeWorldTransformScaled = RelativeTransformTmp * ParticleWorldTransform;
					}
				}
				else if (const TImplicitObjectInstanced<FConvex>* InstancedObject = TImplicitObjectInstanced<FConvex>::AsInstanced(*Implicit))
				{
					Convex = InstancedObject->Object();
				}				

				// We support convex and boxes separately with the same buoyancy algorithm.
				// Convex uses geometric volume for scaling (submerged vol computed against real geometry).
				// Box uses bounding box volume (spheres/other shapes are boxified for submersion).
				if (Convex && !bBuoyancyAlgorithmsUseConvexAsBox)
				{
					FBuoyancyConvexShape ConvexShapeQuery(Convex);
					ConvexShapeQuery.Initialize();

					ComputeSubmergedVolumeAndForcesForShape<FBuoyancyConvexShape, SamplerType>(SubmergedParticle, ConvexShapeQuery,
						Evolution, DeltaSeconds, WaterDensity, WaterDrag, WaterLift,
						ParticleVol, ShapeVolGeom,
						ShapeWorldTransformScaled, WaterSampler,
						OutSubmergedVol, OutSubmergedCoM, OutForce, OutTorque, OutBuoyancyForce, OutBuoyancyTorque);
				}
				else
				{
					FBuoyancyBoxShape BoxShapeQuery(LocalBox);
					BoxShapeQuery.Initialize();

					ComputeSubmergedVolumeAndForcesForShape<FBuoyancyBoxShape, SamplerType>(SubmergedParticle, BoxShapeQuery,
						Evolution, DeltaSeconds, WaterDensity, WaterDrag, WaterLift,
						ParticleVol, ShapeVolBBox,
						ShapeWorldTransform, WaterSampler,
						OutSubmergedVol, OutSubmergedCoM, OutForce, OutTorque, OutBuoyancyForce, OutBuoyancyTorque);
				}

#if CHAOS_DEBUG_DRAW
				if (bBuoyancyDebugDraw)
				{					
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(
						ShapeWorldTransform.TransformPosition(LocalBox.GetCenter()),
						LocalBox.Extents() * .5f,
						ShapeWorldTransform.GetRotation(),
						FColor::Red, false, -1.f, SDPG_Foreground, 3.f);
				}
#endif

				// mark the shape as submerged
				if (OutSubmergedVol > SMALL_NUMBER)
				{
					OutTotalDragForce += OutForce;
					OutTotalDragTorque += OutTorque;
					OutTotalSubmergedCoM += OutSubmergedCoM * OutSubmergedVol;

					OutTotalSubmergedVol += OutSubmergedVol;

					OutTotalBuoyancyForce += OutBuoyancyForce;
					OutTotalBuoyancyTorque += OutBuoyancyTorque;

					SubmergeShape_Internal(SubmergedShapes, ParticleIndex, ObjectIndex);
				}
			});

		// compute final force and torque for particle as weighted average of all the shapes
		if (OutTotalSubmergedVol > UE_SMALL_NUMBER)
		{
			OutTotalSubmergedCoM /= OutTotalSubmergedVol;
		}

		OutTotalParticleVol = ParticleVol;
	}

	// forward declaration of usage
	template void ComputeSubmergedVolumeAndForcesForParticle<FBuoyancyConstantSplineSampler>(FBuoyancyParticleData&,
		const Chaos::FGeometryParticleHandle*, const Chaos::FGeometryParticleHandle*,
		TSharedPtr<FBuoyancyConstantSplineSampler>,
		const Chaos::FPBDRigidsEvolution&, const float, const float, const float, const float,
		float&, float&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&);

	template void ComputeSubmergedVolumeAndForcesForParticle<FBuoyancyShallowWaterSampler>(FBuoyancyParticleData&,
		const Chaos::FGeometryParticleHandle*, const Chaos::FGeometryParticleHandle*,
		TSharedPtr<FBuoyancyShallowWaterSampler>,
		const Chaos::FPBDRigidsEvolution&, const float, const float, const float, const float,
		float&, float&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&);

	template void ComputeSubmergedVolumeAndForcesForParticle<FBuoyancyConstantSplineWithWavesSampler>(FBuoyancyParticleData&,
		const Chaos::FGeometryParticleHandle*, const Chaos::FGeometryParticleHandle*,
		TSharedPtr<FBuoyancyConstantSplineWithWavesSampler>,
		const Chaos::FPBDRigidsEvolution&, const float, const float, const float, const float,
		float&, float&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&, Chaos::FVec3&);

	template <typename ShapeType, typename SamplerType>
	void ComputeSubmergedVolumeAndForcesForShape(const Chaos::FGeometryParticleHandle* SubmergedParticle, const ShapeType& ShapeQuery,
		const Chaos::FPBDRigidsEvolution& Evolution, float DeltaSeconds, const float WaterDensity, const float WaterDrag, const float WaterLift,
		const float ParticleVol, const float ShapeVol,
		const Chaos::FRigidTransform3 &ShapeWorldTransform, TSharedPtr<SamplerType> WaterSampler,
		float& OutSubmergedVol, Chaos::FVec3& OutSubmergedCoM,
		Chaos::FVec3& OutForce, Chaos::FVec3& OutTorque,
		Chaos::FVec3& OutBuoyancyForce, Chaos::FVec3& OutBuoyancyTorque)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape)

		const FConstGenericParticleHandle SubmergedGeneric = SubmergedParticle;		
		const FPBDRigidParticleHandle* RigidParticle = SubmergedParticle->CastToRigidParticle();					

		// water plane parameters for the body.  Note we have 1 plane per box
		Chaos::FVec3 ShapeWaterP = FVec3::ZeroVector;
		Chaos::FVec3 ShapeWaterN = FVec3(0.f, 0.f, 1.f);

		// Compute submerged volume and CoM
		OutSubmergedVol = 0.f;
		OutSubmergedCoM = FVec3::ZeroVector;

		// output forces
		OutForce = FVec3::ZeroVector;
		OutTorque = FVec3::ZeroVector;

		float SubmergedCOMTotalWeight = 0.f;
				
		// either use a water plane per shape's centroid or the closest water point to the particle
		if (WaterSampler->UsePerShapeWaterPlane())
		{			
			const FVector WorldBoxCenter = ShapeWorldTransform.TransformPosition(ShapeQuery.GetCenter());

			// water plane position is computed from the centroid of the box		
			FVector CenterVelocity;
			float CenterHeight;
			float CenterDepth;
			WaterSampler->SampleWaterAtPosition(WorldBoxCenter, CenterVelocity, CenterHeight, CenterDepth);

			// no water at shape centroid of shape so we return
			if (CenterDepth < SMALL_NUMBER)
			{
				return;
			}
						
			ShapeWaterP = WorldBoxCenter;
			ShapeWaterP.Z = CenterHeight;
		}
		else
		{
			ShapeWaterP = WaterSampler->GetClosestWaterPointToParticle();
		}

		// Sample normal at water plane position
		// #todo(dmp): just store normals in the shallow water texture rather than finite differencing here
		ShapeWaterN = WaterSampler->SampleNormalAtPosition(ShapeWaterP);
								
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxVerticesPerShape>> WorldVertexPosition;
		TArray<bool, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxVerticesPerShape>> VertexIsUnderwater;

		const int32 TotalNumVertices = ShapeQuery.NumVertices();

		WorldVertexPosition.AddUninitialized(TotalNumVertices);
		VertexIsUnderwater.AddUninitialized(TotalNumVertices);

		// keep track of a reference point inside the clipped geometry that is the average of all the 
		// submerged vertices and intersection points this will jump around because it isn't weighted by the area/volume
		// but is sufficient for tet volume computations since we only need an interior point
		FVector InteriorRefPoint = FVector::ZeroVector;
		int32 InteriorRefPointCount = 0;
		
		// cache water values for each vertex and determine broadphase of shape interacting with the water	
		{
			SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_ComputeWorld);
			for (int32 i = 0; i < TotalNumVertices; ++i)
			{			
				// compute world space position of vertex
				const FVector CurrWorldVertexPosition = ShapeWorldTransform.TransformPosition(ShapeQuery.GetVertex(i));
				WorldVertexPosition[i] = CurrWorldVertexPosition;

				if ((CurrWorldVertexPosition - ShapeWaterP).Dot(ShapeWaterN) < SMALL_NUMBER)
				{
					VertexIsUnderwater[i] = true;
					InteriorRefPoint += WorldVertexPosition[i];
					InteriorRefPointCount++;				
				}
				else
				{
					VertexIsUnderwater[i] = false;
				}
			}
		}

		// Early out if no vertices are underwater
		if (InteriorRefPointCount == 0)
		{
			return;
		}

		// find intersection points of the box and water plane
		FVector IntersectionCenter;			

		// we can have at most 6 intersection points with a box and a plane			
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxIntersectionPointsPerShape>> OrderedIntersectionPoints;

		const int32 NumEdges = ShapeQuery.NumEdges();
		const int32 MaxBoxPlaneIntersectionsPoints = ShapeQuery.NumMaxIntersectionsPoints();

		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxEdgesPerShape>> EdgeIntersectionPoints;
		EdgeIntersectionPoints.AddUninitialized(NumEdges);

		OrderedIntersectionPoints.AddUninitialized(MaxBoxPlaneIntersectionsPoints);

		int32 NumIntersectionPoints;
		BuoyancyAlgorithms::FindAllIntersectionPoints<ShapeType>(ShapeWaterP, ShapeWaterN, ShapeQuery, WorldVertexPosition, VertexIsUnderwater, EdgeIntersectionPoints, NumIntersectionPoints,
			OrderedIntersectionPoints, IntersectionCenter);		

		// Add intersection points to interior reference point
		// values are already accumulated into the intersection center
		InteriorRefPoint += IntersectionCenter * NumIntersectionPoints;
		InteriorRefPointCount += NumIntersectionPoints;
		InteriorRefPoint /= InteriorRefPointCount;
		
		const FVec3 WorldCoM = SubmergedGeneric->PCom();

		// Particle velocity — constant across all triangles for this shape
		const FVec3 ParticleV = RigidParticle->GetV();
		const FVec3 ParticleW = RigidParticle->GetW();

		// Hoist SubmergedFaceVertices allocation outside the face loop - reuse via Reset() per face
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxSubmergedFaceVertices>> SubmergedFaceVertices;

		// Sum up submerged areas and volumes for each face
		const int32 NumFaces = ShapeQuery.NumFaces();
		for (int32 FaceIdx = 0; FaceIdx < NumFaces; ++FaceIdx)
		{
#if BUOYANCY_INNER_LOOP_STATS
			SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_FindFaceVerticesBelowWater);
#endif

			// Quick pre-check: if no face vertex is underwater, the face is fully dry.
			// With edge early-out, edges only intersect when vertices are on both sides,
			// so no underwater vertex means no edge intersections for this face either.
			const int32 NumVerticesForFace = ShapeQuery.NumFaceVertices(FaceIdx);
			{
				bool bFaceHasUnderwaterVertex = false;
				for (int32 i = 0; i < NumVerticesForFace; ++i)
				{
					if (VertexIsUnderwater[ShapeQuery.GetFaceVertex(FaceIdx, i)])
					{
						bFaceHasUnderwaterVertex = true;
						break;
					}
				}
				if (!bFaceHasUnderwaterVertex)
				{
					continue;
				}
			}

			FVector SubmergedFaceCenter = FVector::ZeroVector;
			SubmergedFaceVertices.Reset();

			for (int32 i = 0; i < NumVerticesForFace; ++i)
			{				
				// get the current edge index belonging to the i and i+1 vertices
				// note this is a bit weird...we could just ask the edge for the vertices but they
				// might be in the wrong order for correctly constructing the intersected face
				const int32 CurrEdgeIdx = ShapeQuery.GetFaceEdge(FaceIdx, i);

				// if there is an intersection for this edge, add it to the list for the current face
				if (CurrEdgeIdx >= 0 && CurrEdgeIdx < EdgeIntersectionPoints.Num())
				{
					const FVector& EdgeIntersection = EdgeIntersectionPoints[CurrEdgeIdx];
					if (EdgeIntersection.X != TNumericLimits<float>::Max())
					{
						SubmergedFaceVertices.Add(EdgeIntersection);
						SubmergedFaceCenter += EdgeIntersection;
					}
				}

				// Endpoint of the edge
				const int32 Edge1Idx = ShapeQuery.GetFaceVertex(FaceIdx, (i + 1) % NumVerticesForFace);

				// add the second endpoint if it is underwater
				if (VertexIsUnderwater[Edge1Idx])
				{
					const FVector& V1 = WorldVertexPosition[Edge1Idx];

					SubmergedFaceVertices.Add(V1);
					SubmergedFaceCenter += V1;
				}
			}
			
			const int32 NumSubmergedFaceVertices = SubmergedFaceVertices.Num();

			// if the face intersects the water or is fully submerged, compute face coverage
			if (NumSubmergedFaceVertices > 0)
			{
#if BUOYANCY_INNER_LOOP_STATS
				SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_ComputeFaceForces);
#endif

				SubmergedFaceCenter /= NumSubmergedFaceVertices;

				// we have a list of the intersection points for a face in correct counterclockwise order
				// fan layout for points on face, sum up areas and volume for the current face
				for (int32 i = 0; i < NumSubmergedFaceVertices; i++)
				{					
					const FVector &V0 = SubmergedFaceVertices[i];
					const FVector &V1 = SubmergedFaceVertices[(i + 1) % NumSubmergedFaceVertices];
					const FVector &V2 = SubmergedFaceCenter;

					FVector TriBaryCenter;
					float TriArea;
					float TetVolume;
					FVector TriNormal;
					BuoyancyAlgorithms::ComputeTriangleAreaAndVolume(V0, V1, V2, InteriorRefPoint, TriBaryCenter, TriNormal, TriArea, TetVolume);

#if CHAOS_DEBUG_DRAW
					if (bBuoyancyDebugDraw)
					{
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(V0, V1, FColor::Orange, false, -1.f, 0, 1.f);
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(V1, V2, FColor::Orange, false, -1.f, 0, 1.f);
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(V2, V0, FColor::Orange, false, -1.f, 0, 1.f);
					}
#endif

					// skip degenerate triangles with 0 area
					if (TriArea < SMALL_NUMBER)
					{
						continue;
					}

					OutSubmergedCoM += TriBaryCenter * TriArea;
					SubmergedCOMTotalWeight += TriArea;
					OutSubmergedVol += TetVolume;

					/////
					// sum up forces/torques on the triangle
					/////								

					// sample velocity at center of triangle
					// #todo(dmp): this should just be interpolated from vertex velocities to avoid so many samples	
					// note this is tricky since there is a decent bit of bookkeeping needed with querying intersection vertices
					FVector CurrWaterVelocity;
					{
#if BUOYANCY_INNER_LOOP_STATS
						SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_SampleVelocity)
#endif
						CurrWaterVelocity = WaterSampler->SampleVelocityAtPosition(TriBaryCenter);
					}

					FVector TotalWorldForce;
					FVector TotalWorldTorque;

					BuoyancyAlgorithms::ComputeFluidForceForTriangle(
						WaterDrag, WaterLift, WaterDensity, DeltaSeconds, ParticleV, ParticleW, WorldCoM, TriBaryCenter, TriNormal, TriArea, TetVolume,
						CurrWaterVelocity, ShapeWaterP, ShapeWaterN, TotalWorldForce, TotalWorldTorque);

					OutForce += TotalWorldForce;
					OutTorque += TotalWorldTorque;					

#if CHAOS_DEBUG_DRAW
					if (bBuoyancyDebugDraw)
					{
						FVector CurrWaterVelocityViz = CurrWaterVelocity;
						CurrWaterVelocityViz.Normalize();
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(TriBaryCenter, TriBaryCenter + CurrWaterVelocityViz * 20, 20, FColor::Red, false, -1.f, -1, 2.f);
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(TriBaryCenter, TriBaryCenter + TotalWorldForce * .000025, 20, FColor::Green, false, -1.f, -1, 2.f);
					}
#endif					
				}
			}
		}

		// Sum up tet volumes from the fan of triangles across the intersection face	
		// note that the unique number of intersection points might differ as some convex hull points can collapse to a single one during the ordering
		// process of the intersection algorithm	
		int32 NumUniqueIntersectionPoints = OrderedIntersectionPoints.Num();
		for (int32 i = 0; i < NumUniqueIntersectionPoints; i++)
		{
			const FVector &V0 = OrderedIntersectionPoints[i];
			const FVector &V1 = OrderedIntersectionPoints[(i + 1) % NumUniqueIntersectionPoints];
			const FVector &V2 = IntersectionCenter;

			FVector TriBaryCenter;
			float TriArea;
			float TetVolume;
			FVector TriNormal;
			BuoyancyAlgorithms::ComputeTriangleAreaAndVolume(V0, V1, V2, InteriorRefPoint, TriBaryCenter, TriNormal, TriArea, TetVolume);

#if CHAOS_DEBUG_DRAW
			if (bBuoyancyDebugDraw)
			{
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(V0, V1, FColor::Orange, false, -1.f, 0, 1.f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(V1, V2, FColor::Orange, false, -1.f, 0, 1.f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(V2, V0, FColor::Orange, false, -1.f, 0, 1.f);
			}
#endif

			OutSubmergedCoM += TriBaryCenter * TriArea;
			SubmergedCOMTotalWeight += TriArea;
			OutSubmergedVol += TetVolume;
		}

		// normalize weighted average of contributions to world submerged COM
		// return if nothing has contributed to CoM	
		if (SubmergedCOMTotalWeight < SMALL_NUMBER)
		{
			return;
		}
					
		OutSubmergedCoM /= SubmergedCOMTotalWeight;

		// compute buoyancy force and delta velocity for solver
		Chaos::FVec3 WorldBuoyantForce = Chaos::FVec3(0.0, 0.0, 0.0);
		Chaos::FVec3 WorldBuoyantTorque = Chaos::FVec3(0.0, 0.0, 0.0);
		BuoyancyAlgorithms::ComputeBuoyantForceForShape(Evolution, RigidParticle, DeltaSeconds,
			WaterDensity, ParticleVol, ShapeVol,
			OutSubmergedCoM, OutSubmergedVol, ShapeWaterN, WorldBuoyantForce, WorldBuoyantTorque);
		
		// track the buoyancy force separately
		OutBuoyancyForce = WorldBuoyantForce;
		OutBuoyancyTorque = WorldBuoyantTorque;

#if CHAOS_DEBUG_DRAW
		if (bBuoyancyDebugDraw)
		{
			FVector WorldBoxCenter = ShapeWorldTransform.TransformPosition(ShapeQuery.GetCenter());

			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(OutSubmergedCoM, OutSubmergedCoM + OutForce * .00005, 20, FColor::Black, false, -1.f, -1, 2.f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(WorldBoxCenter, 10, 10, FColor::Green, false, -1.f, -1, 2.0f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(InteriorRefPoint, 10, 10, FColor::Yellow, false, -1.f, -1, 2.0f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(ShapeWaterP, 10, 10, FColor::Orange, false, -1.f, -1, 2.0f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ShapeWaterP, ShapeWaterP + ShapeWaterN * 100, 20, FColor::Orange, false, -1.f, -1, 6.f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(OutSubmergedCoM, 10, 10, FColor::Magenta, false, -1.f, -1, 2.0f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(OutSubmergedCoM, OutSubmergedCoM + OutForce * .00005, 20, FColor::Magenta, false, -1.f, -1, 2.f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(OutSubmergedCoM, OutSubmergedCoM + OutTorque * .00005, 20, FColor::Red, false, -1.f, -1, 2.f);
		}
#endif
	}

	template <typename ShapeType>
	void FindAllIntersectionPoints(const Chaos::FVec3& WaterP, const Chaos::FVec3& WaterN, const ShapeType&ShapeQuery,
		const TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxVerticesPerShape>> &WorldVertexPosition,
		const TArray<bool, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxVerticesPerShape>> &VertexIsUnderwater,
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxEdgesPerShape>>& EdgeIntersectionPoints, int32 &NumIntersections,
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxIntersectionPointsPerShape>>& OutOrderedIntersectionPoints, FVector& OutIntersectionCenter)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_FindAllIntersectionPoints)

		OutIntersectionCenter = FVector::ZeroVector;
		NumIntersections = 0;

		// Sentinel value for non-intersecting edges
		const FVector SentinelValue(TNumericLimits<float>::Max());

		// loop over all edges and compute intersection points
		const int32 NumEdges = ShapeQuery.NumEdges();
		for (int32 CurrEdgeIdx = 0; CurrEdgeIdx < NumEdges; ++CurrEdgeIdx)
		{
			int32 Index0, Index1;
			ShapeQuery.GetEdgeVertices(CurrEdgeIdx, Index0, Index1);

			// An edge can only intersect the water plane if one vertex is above and one below.
			// Skip edges where both vertices are on the same side - avoids EdgePlaneIntersection math.
			if (VertexIsUnderwater[Index0] == VertexIsUnderwater[Index1])
			{
				EdgeIntersectionPoints[CurrEdgeIdx] = SentinelValue;
				continue;
			}

			// Use a local Chaos::FVec3 (float) since EdgePlaneIntersection takes float precision
			Chaos::FVec3 IntersectionPoint;
			if (EdgePlaneIntersection(WaterP, WaterN, WorldVertexPosition[Index0], WorldVertexPosition[Index1], IntersectionPoint))
			{
				EdgeIntersectionPoints[CurrEdgeIdx] = FVector(IntersectionPoint);
				OutIntersectionCenter += EdgeIntersectionPoints[CurrEdgeIdx];
				NumIntersections++;
			}
			else
			{
				EdgeIntersectionPoints[CurrEdgeIdx] = SentinelValue;
			}
		}

		if (NumIntersections > 0)
		{
			OutIntersectionCenter /= NumIntersections;

			// Sort intersection points by angle around reference point
			BuoyancyAlgorithms::SortIntersectionPointsByAngle(WaterN, OutIntersectionCenter, EdgeIntersectionPoints, NumIntersections, OutOrderedIntersectionPoints);
		}
		else
		{
			OutOrderedIntersectionPoints.Reset(0);
		}
	}

	void SortIntersectionPointsByAngle(const Chaos::FVec3& WaterN, const Chaos::FVec3& IntersectionCenter,
		const TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxEdgesPerShape>>& EdgeIntersectionPoints,
		int32 NumIntersections,
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxIntersectionPointsPerShape>>& OutOrderedIntersectionPoints)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_SortIntersectionPointsByAngle)

		// Sort directly into the output array with a parallel angle array for ordering.
		// Insertion sort is optimal for the typical 3-6 intersection points.
		TArray<double, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxIntersectionPointsPerShape>> Angles;
		Angles.Reserve(NumIntersections);
		OutOrderedIntersectionPoints.Reset();
		OutOrderedIntersectionPoints.Reserve(NumIntersections);

		// first element is assigned to be angle 0
		FVector AngleStart;
		bool FoundFirst = false;

		// loop over all edges, find valid intersection points then determine angle to reference point
		const int32 NumEdges = EdgeIntersectionPoints.Num();
		for (int32 EdgeIdx = 0; EdgeIdx < NumEdges; ++EdgeIdx)
		{
			const FVector& CurrPoint = EdgeIntersectionPoints[EdgeIdx];

			// Skip edges with no intersection (sentinel value)
			if (CurrPoint.X == TNumericLimits<float>::Max())
			{
				continue;
			}

			double CurrAngle;

			if (!FoundFirst) // first point found is the 0 angle point
			{
				AngleStart = CurrPoint - IntersectionCenter;
				AngleStart.Normalize();

				CurrAngle = 0.0;
				FoundFirst = true;
			}
			else // compute angle from the first point to this one
			{
				FVector CurrVector = CurrPoint - IntersectionCenter;
				CurrVector.Normalize();

				CurrAngle = FMath::Acos(FMath::Clamp(AngleStart.Dot(CurrVector), -1.0, 1.0));

				if (AngleStart.Cross(CurrVector).Dot(WaterN) < SMALL_NUMBER)
				{
					CurrAngle = 2.0 * PI - CurrAngle;
				}
			}

			// Insertion sort into both parallel arrays
			int32 InsertIdx = Angles.Num();
			for (int32 j = 0; j < Angles.Num(); ++j)
			{
				if (CurrAngle < Angles[j])
				{
					InsertIdx = j;
					break;
				}
			}
			Angles.Insert(CurrAngle, InsertIdx);
			OutOrderedIntersectionPoints.Insert(CurrPoint, InsertIdx);
		}
	}

	bool EdgePlaneIntersection(const Chaos::FVec3& WaterP, const Chaos::FVec3& WaterN, const Chaos::FVec3& V0, const Chaos::FVec3& V1, Chaos::FVec3& IntersectionPoint)
	{
		// #todo(dmp): we currently use a fixed array with sentinel values for edges that have no intersection point
		// we could use a bit mask or sparse array instead.  We switched from using a map due to perf
		IntersectionPoint = FVector(TNumericLimits<float>::Max());

		// edge plane intersection for each edge in counterclockwise order
		const FVector& RayOrigin = V0;
		const FVector RayDir = V1 - V0;

		// if the water plane intersects the current edge, add intersection point it to the list
		const float DirDotN = RayDir.Dot(WaterN);

		if (FMath::Abs(DirDotN) > SMALL_NUMBER)
		{
			const float t = (WaterP - RayOrigin).Dot(WaterN) / DirDotN;
			if (t >= 0 && t <= 1)
			{
				IntersectionPoint = RayOrigin + RayDir * t;

				return true;
			}
		}

		return false;
	}	

	void ComputeBuoyantForceForShape(const Chaos::FPBDRigidsEvolution& Evolution, const Chaos::FPBDRigidParticleHandle* RigidParticle, const float DeltaSeconds, const float WaterDensity,
		const float ParticleVol, const float ShapeVol,
		const Chaos::FVec3& SubmergedCoM, const float SubmergedVol, const Chaos::FVec3& WaterN, Chaos::FVec3& OutWorldBuoyantForce, Chaos::FVec3& OutWorldBuoyantTorque)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_ComputeBuoyantForceForShape)

		// Get perparticle gravity rule, for figuring out the effective gravity on buoyant objects
		const Chaos::FPerParticleGravity* PerParticleGravity = &Evolution.GetGravityForces();

		// Figure out the gravity level of the particle
		const int32 GravityGroupIndex = RigidParticle->GravityGroupIndex();
		const Chaos::FVec3 GravityAccelVec
			= PerParticleGravity != nullptr && GravityGroupIndex != INDEX_NONE
			? (Chaos::FVec3) PerParticleGravity->GetAcceleration(GravityGroupIndex)
			: Chaos::FVec3::DownVector * 980.f; // Default to "regular" gravity

		// NOTE: We assume gravity is -Z for perf... If we want to support buoyancy for
		// weird gravity setups, this is where we'd have to fix it up.
		const FVec3 GravityDir = FVec3::DownVector;
		const float GravityAccel = FVec3::DotProduct(GravityDir, GravityAccelVec);

		// Scale submerged volume using precomputed particle/shape volumes
		float ScaledSubmergedVol = SubmergedVol;
		float TotalParticleVol;
		ScaleSubmergedVolume(ParticleVol, ShapeVol, ScaledSubmergedVol, TotalParticleVol);

		// Compute buoyant force
		//
		// NOTE: This is easy to compute with Archimedes' principle
		// https://en.wikipedia.org/wiki/Buoyancy
		//
		const float BuoyantForceMagnitude = WaterDensity * ScaledSubmergedVol * GravityAccel;
		//                       = [ kg / cm^3 ] * [ cm^3 ]    * [cm / s^2]
		//                       = [ kg * cm / s^2 ]
		//                       = [ force ]

		// Only proceed if buoyant force isn't vanishingly small
		if (BuoyantForceMagnitude < SMALL_NUMBER)
		{
			return;
		}

		// Get a generic particle wrapper
		FConstGenericParticleHandle RigidGeneric(RigidParticle);

		const FVec3 WorldCoM = RigidGeneric->PCom();
		const FVec3 CoMDiff = SubmergedCoM - WorldCoM;
		
		if (bBuoyancyAlgorithmsAccurateBuoyancyAlongNormal)
		{
			OutWorldBuoyantForce = WaterN * BuoyantForceMagnitude;
		}
		else
		{
			OutWorldBuoyantForce = -1.0 * GravityDir * BuoyantForceMagnitude;
		}

		// Compute world buoyant force and torque	
		// buoyancy is computed at the submerged center of mass		
		OutWorldBuoyantTorque = Chaos::FVec3::CrossProduct(CoMDiff, OutWorldBuoyantForce);				
	}

	void FBuoyancyConvexShape::Initialize()
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyAlgorithms_FBuoyancyConvexShape_Initialize)

		// #todo(dmp): ideally the convex would have a mapping between half edges and edges that is serialized, but instead for now we'll build it here
		const int32 NumEdges = Convex->NumEdges();
		HalfEdgeToEdge.AddUninitialized(Convex->GetStructureData().NumHalfEdges());

		for (int32 CurrEdgeIdx = 0; CurrEdgeIdx < NumEdges; ++CurrEdgeIdx)
		{
			int32 HalfEdge0, HalfEdge1;
			Convex->GetHalfEdges(CurrEdgeIdx, HalfEdge0, HalfEdge1);

			// only process valid half edges
			if (HalfEdge0 != INDEX_NONE)
			{
				HalfEdgeToEdge[HalfEdge0] = CurrEdgeIdx;
			}

			if (HalfEdge1 != INDEX_NONE)
			{
				HalfEdgeToEdge[HalfEdge1] = CurrEdgeIdx;
			}
		}
	}

	}