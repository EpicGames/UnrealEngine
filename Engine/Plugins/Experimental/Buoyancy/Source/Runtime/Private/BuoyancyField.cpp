// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyField.h"
#include "HAL/IConsoleManager.h"
#include "BuoyancySubsystem.h"
#include "BuoyancyAlgorithms.h"
#include "Engine/World.h"
#include "DrawDebugHelpers.h"
#include "Chaos/DebugDrawQueue.h"

using namespace Chaos;
using namespace Chaos::Softs;

extern bool bBuoyancyDebugDraw;

namespace Buoyancy
{

namespace Private
{
	static float VelocityFieldMaxVelocity = 0.f;
	static FAutoConsoleVariableRef CVarChaosVelocityFieldMaxVelocity(TEXT("p.Buoyancy.BuoyancyField.MaxVelocity"), VelocityFieldMaxVelocity, TEXT("The maximum relative velocity to process the aerodynamics forces with."));

	// #todo(dmp): resolve threading issues so this can work with Chaos Cloth.  See notes below.  For now, this
	// will be left unused and instead we'll only do thread safe GT queries
	bool bComputeWaterPerTri = false;
	// FAutoConsoleVariableRef CVarBuoyancyAlgorithmsFieldComputeWaterPerTri(TEXT("p.Buoyancy.Algorithms.Field.ComputeWaterPerTri"), bComputeWaterPerTri, TEXT(""));

	bool bUseAccurateSubmersion = false;
	FAutoConsoleVariableRef CVarBuoyancyAlgorithmsFieldUseAccurateSubmersion(TEXT("p.Buoyancy.Algorithms.Field.UseAccurateSubmersion"), bUseAccurateSubmersion, TEXT(""));

}

void FBuoyancyField::SetPropertiesAndBuoyancy(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	FSolverReal WorldScale,
	bool bEnableAerodynamics,
	const FSolverVec3& SolverWind,
	const FRotation3& LocalSpaceRotation,
	const Chaos::Softs::FSolverReal InLocalSpaceScale,
	const Chaos::FSolverVec3 InLocalSpaceLocation,
	const FRotation3& ReferenceSpaceRotation)
{
	SetPropertiesAndWind(PropertyCollection, WeightMaps, WorldScale, bEnableAerodynamics, SolverWind, LocalSpaceRotation, ReferenceSpaceRotation);
	
	// if none are set, then we disable buoyancy
	bUseWaterBodies = 
		ClothDensityInWaterIndex != INDEX_NONE && 
		WaterDensityIndex != INDEX_NONE && 
		WaterTurbulenceRatioIndex != INDEX_NONE;

	ClothDensityInWater = ClothDensityInWaterIndex != INDEX_NONE ? float(GetClothDensityInWater(PropertyCollection)) : 0.1f;	
	
	// #todo(dmp): note the Buoyancy plugin uses g/cm^3 for water density, so this would ideally be unified.
	WaterDensity = WaterDensityIndex != INDEX_NONE ? float(GetWaterDensity(PropertyCollection)) : 1000.f;	
			
	WaterTurbulenceRatio = WaterTurbulenceRatioIndex != INDEX_NONE ? float(GetWaterTurbulenceRatio(PropertyCollection)) : 0.f;	

	// convert to kg/cm^3
	ClothRho = FMath::Max(ClothDensityInWater / FMath::Cube(WorldScale), (FSolverReal)0.);
	WaterRho = FMath::Max(WaterDensity / FMath::Cube(WorldScale), (FSolverReal)0.);
	
	constexpr FSolverReal OneQuarter = (FSolverReal)0.25;
	QuarterWaterRho = WaterRho * OneQuarter;

	LocalSpaceScale = InLocalSpaceScale;
	LocalSpaceLocation = InLocalSpaceLocation;
}

void FBuoyancyField::PreSimulate_GameThread(float DeltaTime, const UWorld* World, const FBox BoundingBox)
{
	check(IsInGameThread());
	
	TRACE_CPUPROFILER_EVENT_SCOPE(FBuoyancyField_PreSimulate_GameThread);

	UBuoyancySubsystem* BuoyancySubsystem = World->GetSubsystem<UBuoyancySubsystem>();
	if (bUseWaterBodies && BuoyancySubsystem)
	{
		// given the bounding box, find the colliding water bodies
		TArray<const TSharedPtr<FBuoyancyWaterSplineData>> WaterBodyCollisionData;

		FVector WaterPlaneLocation = {0,0,0};
		FVector WaterPlaneNormal = {0,0,0};
		FVector WaterPlaneVelocity = {0,0,0};

		bool ComputePlane = !Private::bComputeWaterPerTri;

		bool HasCollision = BuoyancySubsystem->FindOverlappingWaterBodies(BoundingBox, ComputePlane,
			WaterPlaneLocation,
			WaterPlaneNormal,
			WaterPlaneVelocity,
			WaterBodyCollisionData);

		WaterBodyCollisions = FBuoyancyCollisionData(BuoyancySubsystem, WaterBodyCollisionData, BoundingBox,
			HasCollision,
			ComputePlane,
			WaterPlaneLocation,
			WaterPlaneNormal,
			WaterPlaneVelocity);
	}
}

void FBuoyancyField::Apply(FSolverParticlesRange& InParticles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBuoyancyField_Apply);
			
	if (!bUseWaterBodies || WaterBodyCollisions.TheBuoyancySubsystem == nullptr || !WaterBodyCollisions.HasCollision)
	{
		FVelocityAndPressureField::Apply(InParticles, Dt);
		return;
	}
	else
	{
		const FSolverReal MaxVelocitySquared = (Private::VelocityFieldMaxVelocity > 0.f) ? FMath::Square((FSolverReal)Private::VelocityFieldMaxVelocity) : TNumericLimits<FSolverReal>::Max();

		// update according to water bodies
		const TConstArrayView<FSolverVec3>& Xs = InParticles.XArray();
		const TConstArrayView<FSolverVec3>& Vs = InParticles.GetV();
		
		// compute water properties at tri center
		FVector WaterVelocity;
		FVector WaterN;
		FVector WaterPlanePos;

		TSharedPtr<FBuoyancyWaterSplineData> WaterBodySplineData = nullptr;

		if (!Private::bComputeWaterPerTri)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FBuoyancyField_Apply_ComputeWaterPerCloth);
			const FVector WorldAvgX = WaterBodyCollisions.WorldBounds.GetCenter();

			WaterVelocity = WaterBodyCollisions.WaterPlaneVelocity;
			WaterN = WaterBodyCollisions.WaterPlaneNormal;
			WaterPlanePos = WaterBodyCollisions.WaterPlaneLocation;			
		}
		else if (WaterBodyCollisions.WaterBodyCollisionData.Num() > 0)
		{
			// #todo(dmp): we aren't compositing submersions from multiple simultaneous bodies right now- just take the first one
			WaterBodySplineData = WaterBodyCollisions.WaterBodyCollisionData[0];
		}
		else
		{
			FVelocityAndPressureField::Apply(InParticles, Dt);
			return;
		}

		// loop over particles and compute buoyancy forces
		for (int32 Index = 0; Index < InParticles.GetRangeSize(); ++Index)
		{
			if (InParticles.InvM(Index) != (FSolverReal) 0. && ClothRho > (FSolverReal) 0.)
			{				
				const FSolverVec3 WorldPosition = InParticles.GetP(Index) * LocalSpaceScale + LocalSpaceLocation;
			
				// optionally compute water properties at tri center			
				if (Private::bComputeWaterPerTri)
				{				
					TRACE_CPUPROFILER_EVENT_SCOPE(FBuoyancyField_Apply_ComputeWaterPerTri);
								
					const FVector WorldPositionFVector = FVector(WorldPosition);
					WaterBodyCollisions.TheBuoyancySubsystem->QueryWaterBody(WorldPositionFVector, WaterBodySplineData, WaterVelocity, WaterN, WaterPlanePos);
				}

				const FSolverVec3 WaterNSolverVec = FSolverVec3(WaterN);

				if ((WorldPosition - WaterPlanePos).Dot(WaterNSolverVec) < SMALL_NUMBER)
				{
					const FSolverVec3 TriGravity = ExternalForces->GetScaledGravity(Index) * LocalSpaceScale;
									
					// the fraction of the current body (ie: particle) is submerged.  We
					// treat particles as either fully submerged or not.  It could be
					// TriSubmersionArea / TriArea for instance if we did this on triangles
					const FSolverReal SubmersionFraction = 1.;
				
					// fraction of the particle
					// volume = mass / density - scale mass by submersion fraction then divide by rho
					const FSolverReal SubmergedTriVolume = (InParticles.M(Index) * SubmersionFraction) / ClothRho;

					// Archimedes Principle						      cm/sec^2  * kg/cm^3  *     cm^3
					const FSolverVec3 Force = ((FSolverReal) -1.0) * TriGravity * WaterRho * SubmergedTriVolume;

					// Update acceleration
					InParticles.Acceleration(Index) += InParticles.InvM(Index) * Force;
				}
			}
		}

		// loop over triangles and compute drag forces
		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
		{
			const TVec3<int32>& Element = Elements[ElementIndex];
			
			// lift and drag
			const FSolverReal CdI = Drag.GetValue(ElementIndex);
			const FSolverReal CdO = OuterDrag.GetValue(ElementIndex);
			const FSolverReal ClI = Lift.GetValue(ElementIndex);
			const FSolverReal ClO = OuterLift.GetValue(ElementIndex);
			const FSolverReal Cp = Pressure.GetValue(ElementIndex);

			// Compute triangle barycentric center
			const FSolverVec3 WorldTriPositions[3] = 
			{
				Xs[Element[0]] * LocalSpaceScale + LocalSpaceLocation,
				Xs[Element[1]] * LocalSpaceScale + LocalSpaceLocation,
				Xs[Element[2]] * LocalSpaceScale + LocalSpaceLocation
			};

			const FVector WorldTriCenter = FVector((WorldTriPositions[0] + WorldTriPositions[1] + WorldTriPositions[2]) / 3.f);

			// optionally compute water properties at tri center			
			if (Private::bComputeWaterPerTri)
			{				
				TRACE_CPUPROFILER_EVENT_SCOPE(FBuoyancyField_Apply_ComputeWaterPerTri);
												
				WaterBodyCollisions.TheBuoyancySubsystem->QueryWaterBody(WorldTriCenter, WaterBodySplineData, WaterVelocity, WaterN, WaterPlanePos);
			}

			const FSolverVec3 WaterNSolverVec = FSolverVec3(WaterN);
			const FSolverReal TriVertDotWaterPlane[3] =
			{
				(WorldTriPositions[0] - WaterPlanePos).Dot(WaterNSolverVec),
				(WorldTriPositions[1] - WaterPlanePos).Dot(WaterNSolverVec),
				(WorldTriPositions[2] - WaterPlanePos).Dot(WaterNSolverVec)
			};

			// true if the vertex is submerged under the water plane
			const bool TriVertSubmerged[3] =
			{
				TriVertDotWaterPlane[0] < SMALL_NUMBER,
				TriVertDotWaterPlane[1] < SMALL_NUMBER,
				TriVertDotWaterPlane[2] < SMALL_NUMBER
			};

			// early out with no vertices submerged
			// consider tri as above water and use normal wind and drag
			if (!TriVertSubmerged[0] && !TriVertSubmerged[1] && !TriVertSubmerged[2])
			{				
				const FSolverVec3 Force = FVelocityAndPressureField::CalculateForce(Xs, Vs, ElementIndex);
				InParticles.Acceleration(Elements[ElementIndex][0]) += InParticles.InvM(Elements[ElementIndex][0]) * Force;
				InParticles.Acceleration(Elements[ElementIndex][1]) += InParticles.InvM(Elements[ElementIndex][1]) * Force;
				InParticles.Acceleration(Elements[ElementIndex][2]) += InParticles.InvM(Elements[ElementIndex][2]) * Force;

				continue;
			}
			
			// Force to apply to triangle
			FSolverVec3 Force = FSolverVec3::ZeroVector;

			// we have submersion if there is an intersection or the tri center is below the water plane
			// proceed with at least 1 vertex submerged. It can either be fully submerged or intersecting the water surface
			const bool TriIsFullySubmerged = TriVertSubmerged[0] && TriVertSubmerged[1] && TriVertSubmerged[2];			
						
			// Calculate the normal and the area of the surface exposed to the flow
			FSolverVec3 TriNormal = FSolverVec3::CrossProduct(
				Xs[Element[2]] - Xs[Element[0]],
				Xs[Element[1]] - Xs[Element[0]]);

			const FSolverReal NormalLength = TriNormal.SafeNormalize();
			const FSolverReal TriArea = (FSolverReal) .5 * NormalLength;
			FSolverReal SubmergedTriArea;

			// In simple submersion, any triangle that is submerged is considered fully submerged
			if (NormalLength < SMALL_NUMBER)
			{
				continue;
			}
			else if (!Private::bUseAccurateSubmersion || TriIsFullySubmerged)
			{
				// #todo(dmp): simple submersion could be improved by computing the circle circumscribed by the tri then
				// doing analytical submersion depth based on sphere/plane penetration depth
				// or
				// take the bounding box of submerged/intersection points and compute area as a coarse approx
				// or
				// take spheres with radius R at each vertex and compute penetration depth of each one and sum up
				// lots of ways to do this, but plenty of errors due to stretched out triangles misrepresenting the tri area
				// and it is unclear if it'd be faster than the accurate geometric method below
				
				SubmergedTriArea = TriArea;
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FBuoyancyField_Apply_Accurate_Submersion);

				const bool TriVertOnWaterPlane[3] =
				{
					TriVertSubmerged[0] && TriVertDotWaterPlane[0] > -SMALL_NUMBER,
					TriVertSubmerged[1] && TriVertDotWaterPlane[1] > -SMALL_NUMBER,
					TriVertSubmerged[2] && TriVertDotWaterPlane[2] > -SMALL_NUMBER
				};
				
				// Triangle is partially submerged, compute intersection area

				// vertices composing the submerged polygon (3 or 4 for a triangle)
				Chaos::FVec3 SubmergedVerts[4];								
				int NumSubmergedVerts = 0;
				
				// compute the ordered submerged vertices
				for (int i = 0; i < 3; ++i)
				{
					if (TriVertSubmerged[i])
					{
						SubmergedVerts[NumSubmergedVerts++] = WorldTriPositions[i];
					}
						
					// #todo(dmp) inline?
					Chaos::FVec3 EdgeIntersection;
					if (!TriVertOnWaterPlane[(i+1) % 3] && BuoyancyAlgorithms::EdgePlaneIntersection(WaterPlanePos, WaterN, WorldTriPositions[i], WorldTriPositions[(i+1) % 3], EdgeIntersection))						
					{
						SubmergedVerts[NumSubmergedVerts++] = EdgeIntersection;
					}
				}

				// compute the submerged volume constructed as 1 or two triangles
				// num submerged is either 3 (submerged triangle) or 4 (submerged quad).  
				// otherwise we have a degenerate case
				if (NumSubmergedVerts < 3)
				{
					continue;
				}

				// Compute area of first triangle
				SubmergedTriArea = (FSolverReal) .5 * 
					(SubmergedVerts[2] - SubmergedVerts[0]).Cross(
					SubmergedVerts[1] - SubmergedVerts[0]).Length();
					
				// two triangle that span the submerged part, add the second one
				if (NumSubmergedVerts == 4)
				{
					SubmergedTriArea += (FSolverReal) .5 * 
						(SubmergedVerts[2] - SubmergedVerts[0]).Cross(
						SubmergedVerts[3] - SubmergedVerts[0]).Length();
				}					
			}						
			
			// project water velocity onto water plane
			const FSolverVec3 WaterVelocityOnPlane = WaterVelocity - WaterVelocity.Dot(WaterN) * WaterN;
										
			// Tri velocity average of 3 verts
			const FSolverVec3& TriSurfaceVelocity = (FSolverReal)(1. / 3.) * (
				Vs[Element[0]] +
				Vs[Element[1]] +
				Vs[Element[2]]);
			
			Force = CalculateForce(TriNormal, TriSurfaceVelocity, SubmergedTriArea, WaterVelocityOnPlane, CdI, CdO, ClI, ClO, Cp, MaxVelocitySquared);

			// increment acceleration on each vertex
			// #todo(dmp): only update vertices that are submerged or whole triangle?
			InParticles.Acceleration(Element[0]) += InParticles.InvM(Element[0]) * Force;
			InParticles.Acceleration(Element[1]) += InParticles.InvM(Element[1]) * Force;
			InParticles.Acceleration(Element[2]) += InParticles.InvM(Element[2]) * Force;

#if CHAOS_DEBUG_DRAW
			if (bBuoyancyDebugDraw)
			{
				const FVector WorldAvgX = WaterBodyCollisions.WorldBounds.GetCenter();

				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(WorldTriCenter, WorldTriCenter + FVector(TriNormal)*10, 1, FColor::Black, false, -1.f, -1, .25f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(WaterPlanePos, 3, 10, FColor::Orange, false, -1.f, -1, 2.0f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(WorldAvgX, 3, 10, FColor::Blue, false, -1.f, -1, 2.0f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(WaterPlanePos, WaterPlanePos + WaterN * 100, 20, FColor::Orange, false, -1.f, -1, 6.f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(WaterPlanePos, WaterPlanePos + WaterVelocity * 2, 1, FColor::Green, false, -1.f, -1, 6.f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(WorldTriCenter, WorldTriCenter + FVector(Force), 1, FColor::Blue, false, -1.f, -1, 1.f);
			}
#endif				
		}
	}
}
}