// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyParticleData.h"

bool bBuoyancyParticleDataHashIndex = true;
FAutoConsoleVariableRef CVarBuoyancyParticleDataHashIndex(TEXT("p.Buoyancy.ParticleData.HashIndex"), bBuoyancyParticleDataHashIndex, TEXT("If true, use an FHashTable to index particle data maps."));

float BuoyancyParticleDataHashSizeRatioGrowThreshold = 1.5f;
FAutoConsoleVariableRef CVarBuoyancyParticleDataHashSizeRatioGrowThreshold(TEXT("p.Buoyancy.ParticleData.HashSizeRatioGrowThreshold"), BuoyancyParticleDataHashSizeRatioGrowThreshold, TEXT("If the ratio of index size to hash size passes this threshold, a rehash will occur to optimize memory. Only occurs if bBuoyancyParticleDataHashIndex is true. A value of less than 1 will result in a sparser hash table, a value of greater than 1 results in a denser table. A value of 0 or less will cause the optimization to be skipped. No value can guarantee zero hash collisions."));

int32 BuoyancyParticleDataMinHashSize = 1024;
FAutoConsoleVariableRef CVarBuoyancyParticleDataMinHashSize(TEXT("p.Buoyancy.ParticleData.MinHashSize"), BuoyancyParticleDataMinHashSize, TEXT("Hash size is not allowed to shrink below this size. Must be a power of 2!"));


int32 BuoyancyParticleDataMaxHashSize = 8192;
FAutoConsoleVariableRef CVarBuoyancyParticleDataMaxHashSize(TEXT("p.Buoyancy.ParticleData.MaxHashSize"), BuoyancyParticleDataMaxHashSize, TEXT("Hash size is not allowed to grow beyond this size. Must be a power of 2!"));

FBuoyancyParticleData::FBuoyancyParticleData()
	: IndexMap(BuoyancyParticleDataMinHashSize)
{ }

FBuoyancyParticleData::~FBuoyancyParticleData()
{ }

void FBuoyancyParticleData::Reset()
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancyParticleData_Reset)
	LLM_SCOPE_BYTAG(BuoyancyParticleDataTag);

	IndexMap.Clear();
	ReverseIndexMap.Reset();
	Interactions.Reset();
	Submersions.Reset();
	PrevSubmersions.Reset();
	SubmersionMetaData.Reset();
	PrevSubmersionMetaData.Reset();
	SubmergedShapes.Reset();
}

SIZE_T FBuoyancyParticleData::GetAllocatedSize() const
{
	return
		IndexMap.GetAllocatedSize() +
		ReverseIndexMap.GetAllocatedSize() +
		Interactions.GetAllocatedSize() +
		Submersions.GetAllocatedSize() + 
		PrevSubmersions.GetAllocatedSize() +
		SubmersionMetaData.GetAllocatedSize() +
		PrevSubmersionMetaData.GetAllocatedSize() +
		SubmergedShapes.GetAllocatedSize();
}

int32 FBuoyancyParticleData::GetIndex(const Chaos::FGeometryParticleHandle& ParticleHandle)
{
	int32 ParticleIndex, ParticleKey;
	return GetIndex(ParticleHandle, ParticleIndex, ParticleKey);
}

int32 FBuoyancyParticleData::GetIndex(const Chaos::FGeometryParticleHandle& ParticleHandle, int32& OutParticleIndex, int32& OutParticleKey)
{
	OutParticleIndex = ParticleHandle.UniqueIdx().Idx;

	// If we're not hashing indices here, return the particle index itself
	if (bBuoyancyParticleDataHashIndex == false)
	{
		OutParticleKey = INDEX_NONE;
		return OutParticleIndex;
	}

	OutParticleKey = MurmurFinalize32(OutParticleIndex);
	return GetIndex(OutParticleIndex, OutParticleKey);
}

int32 FBuoyancyParticleData::GetIndex(const int32 ParticleIndex, const int32 ParticleKey)
{
	for (uint32 DataIndex = IndexMap.First(ParticleKey); IndexMap.IsValid(DataIndex); DataIndex = IndexMap.Next(DataIndex))
	{
		if (ReverseIndexMap[DataIndex] == ParticleIndex)
		{
			return DataIndex;
		}
	}

	return INDEX_NONE;
}

int32 FBuoyancyParticleData::AddOrGetIndex(const Chaos::FGeometryParticleHandle& ParticleHandle)
{
	LLM_SCOPE_BYTAG(BuoyancyParticleDataTag);

	const int32 ParticleIndex = ParticleHandle.UniqueIdx().Idx;

	// If we're not hashing indices here, return the particle index itself
	if (bBuoyancyParticleDataHashIndex == false)
	{
		return ParticleIndex;
	}

	const int32 ParticleKey = MurmurFinalize32(ParticleIndex);
	int32 DataIndex = GetIndex(ParticleIndex, ParticleKey);
	if (DataIndex == INDEX_NONE)
	{
		DataIndex = ReverseIndexMap.Add(ParticleIndex);
		IndexMap.Add(ParticleKey, DataIndex);
		OptimizeMemory();
	}

	return DataIndex;
}

bool FBuoyancyParticleData::RemoveIndex(const Chaos::FGeometryParticleHandle& ParticleHandle)
{
	LLM_SCOPE_BYTAG(BuoyancyParticleDataTag);

	int32 ParticleIndex, ParticleKey;
	const int32 DataIndex = GetIndex(ParticleHandle, ParticleIndex, ParticleKey);

	// Remove the key from our index map
	IndexMap.Remove(ParticleKey, DataIndex);

	// Remove the data index from all sparse arrays
	ReverseIndexMap.RemoveAt(DataIndex);
	Interactions.RemoveAt(DataIndex);
	Submersions.RemoveAt(DataIndex);
	PrevSubmersions.RemoveAt(DataIndex);
	SubmersionMetaData.RemoveAt(DataIndex);
	PrevSubmersionMetaData.RemoveAt(DataIndex);
	SubmergedShapes.RemoveAt(DataIndex);

	OptimizeMemory();

	return true;
}

void FBuoyancyParticleData::OptimizeMemory()
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancyParticleData_OptimizeMemory)
	LLM_SCOPE_BYTAG(BuoyancyParticleDataTag);

	// Shrink all maps to fit data
	ReverseIndexMap.Shrink();
	Interactions.Shrink();
	Submersions.Shrink();
	PrevSubmersions.Shrink();
	SubmersionMetaData.Shrink();
	PrevSubmersionMetaData.Shrink();
	SubmergedShapes.Shrink();

	if (bBuoyancyParticleDataHashIndex == false)
	{
		return;
	}

	const float HashSizeRatioThreshold = BuoyancyParticleDataHashSizeRatioGrowThreshold;
	if (HashSizeRatioThreshold <= 0.f)
	{
		return;
	}

	// Compute the actual hash size ratio - if it hasn't passed the threshold,
	// nothing to do.
	const float IndexSize = IndexMap.GetIndexSize();
	const float HashSize = IndexMap.GetHashSize();
	const float HashSizeRatio = IndexSize / FMath::Max(1.f, HashSize);
	if (HashSizeRatio <= HashSizeRatioThreshold)
	{
		return;
	}

	// Find the smallest power of two which is greater than IndexSize. If it didn't
	// change, do nothing.
	const int32 MinHashSize = BuoyancyParticleDataMinHashSize;
	const int32 MaxHashSize = BuoyancyParticleDataMaxHashSize;
	const int32 NewRawHashSize = FMath::Pow(2, FMath::CeilToFloat(FMath::Log2(IndexSize)));
	const int32 NewHashSize = FMath::Clamp(NewRawHashSize, MinHashSize, MaxHashSize);
	if (HashSize == NewHashSize)
	{
		return;
	}

	// If we have a new hash size, reallocate our IndexMap with our new hash size, and
	// repopulate it using the reverse index map data.
	IndexMap.Clear(NewHashSize, IndexSize);
	for (auto It = ReverseIndexMap.CreateConstIterator(); It; ++It)
	{
		const int32 DataIndex = It.GetIndex();
		const int32 ParticleIndex = *It;
		const int32 ParticleKey = MurmurFinalize32(ParticleIndex);
		IndexMap.Add(ParticleKey, DataIndex);
	}
}

