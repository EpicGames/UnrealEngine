// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyRuntimeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuoyancyRuntimeSettings)

FName UBuoyancyRuntimeSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}

void UBuoyancyRuntimeSettings::PostInitProperties()
{
	Super::PostInitProperties();
}

#if WITH_EDITOR

UBuoyancyRuntimeSettings::FOnUpdateSettings UBuoyancyRuntimeSettings::OnSettingsChange;

void UBuoyancyRuntimeSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnSettingsChange.Broadcast(this, PropertyChangedEvent.ChangeType);
}

#endif // WITH_EDITOR
