// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyStats.h"

LLM_DEFINE_TAG(BuoyancyParticleDataTag, TEXT("BuoyancyParticleData"), TEXT("Physics"));

