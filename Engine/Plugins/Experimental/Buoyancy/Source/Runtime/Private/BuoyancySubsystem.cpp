// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancySubsystem.h"
#include "BuoyancyAlgorithms.h"
#include "BuoyancyStats.h"
#include "BuoyancyRuntimeSettings.h"
#include "BuoyancyEventInterface.h"
#include "Engine/World.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/PhysicsFiltering.h"
#include "Chaos/MidPhaseModification.h"
#include "Chaos/MassProperties.h"
#include "DrawDebugHelpers.h"
#include "WaterBodyActor.h"
#include "Components/SplineComponent.h"
#include "WaterSubsystem.h"
#include "WaterBodyManager.h"
#include "WaterSplineComponent.h"
#include "BakedShallowWaterSimulationComponent.h"
#include "Chaos/PhysicsObject.h"
#include "PBDRigidsSolver.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/DebugDrawQueue.h"
#include "Templates/SharedPointer.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/GeometryParticlesfwd.h"

#include "Chaos/ParticleHandle.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Collision/CollisionFilter.h"
#include "Chaos/Collision/CollisionUtil.h"
#include "Chaos/Sphere.h"
#include "Engine/OverlapResult.h"
//
// CVars
//

// Jira to remove: PLAY-21231

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuoyancySubsystem)
bool bBuoyancyCallbackDataEnabled = true;
FAutoConsoleVariableRef CVarBuoyancyCallbackDataEnabled(TEXT("p.Buoyancy.CallbackData.Enabled"), bBuoyancyCallbackDataEnabled, TEXT(""));

// Added for jira PLAY-37027
bool bBuoyancyCallbackDataParticleValidation = true;
FAutoConsoleVariableRef CVarBuoyancyCallbackDataParticleValidation(TEXT("p.Buoyancy.CallbackData.ParticleValidation"), bBuoyancyCallbackDataParticleValidation, TEXT(""));

bool bBuoyancyDebugDraw = false;
#if CHAOS_DEBUG_DRAW
FAutoConsoleVariableRef CVarBuoyancyDebugDraw(TEXT("p.Buoyancy.DebugDraw"), bBuoyancyDebugDraw, TEXT(""));
#endif


int32 bUseShallowWaterSimulation = 1;
static FAutoConsoleVariableRef CVarUseShallowWaterSimulation(
	TEXT("p.Buoyancy.bUseShallowWaterSimulation"),
	bUseShallowWaterSimulation,
	TEXT("Accurate buoyancy method"),
	ECVF_Scalability
);

bool bBuoyancyUseWaves = true;
static FAutoConsoleVariableRef CVarBuoyancyUseWaves(
	TEXT("p.Buoyancy.bUseWaves"),
	bBuoyancyUseWaves,
	TEXT("When true, buoyancy applies Gerstner wave displacement for water bodies that have waves"),
	ECVF_Scalability
);

bool bBuoyancyUseSimpleWaves = false;
static FAutoConsoleVariableRef CVarBuoyancyUseSimpleWaves(
	TEXT("p.Buoyancy.bUseSimpleWaves"),
	bBuoyancyUseSimpleWaves,
	TEXT("When true, use the faster simple wave height evaluation (height only, no normal) instead of the full two-sample steepness evaluation"),
	ECVF_Scalability
);

bool bBuoyancyUseImplicitDrag = true;
static FAutoConsoleVariableRef CVarBuoyancyUseImplicitDrag(
	TEXT("p.Buoyancy.UseImplicitDrag"),
	bBuoyancyUseImplicitDrag,
	TEXT("When true, use semi-implicit integration for drag forces to prevent overshoot at high drag values. When false, use explicit Euler for all forces."),
	ECVF_Scalability
);

int32 bUseAccurateIntegrationForSplines = 0;
static FAutoConsoleVariableRef CVarUseAccurateIntegrationForSplines(
	TEXT("p.Buoyancy.bUseAccurateIntegrationForSplines"),
	bUseAccurateIntegrationForSplines,
	TEXT("Accurate buoyancy method for splines"),
	ECVF_Scalability
);

//
// Logging
//

DEFINE_LOG_CATEGORY(LogBuoyancySubsystem);


//
// Console Commands
//

#if WITH_BUOYANCY_MEMORY_TRACKING
static FAutoConsoleCommandWithWorld BuoyancyLogMemory(
	TEXT("p.Buoyancy.LogMemory"),
	TEXT("Log how much memory is being used by the buoyancy subsystem"),
	FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* World)
	{
		if (UBuoyancySubsystem* BuoyancySubsystem = World != nullptr ? World->GetSubsystem<UBuoyancySubsystem>() : nullptr)
		{
			const int32 AllocatedSize = BuoyancySubsystem->GetAllocatedSize();
			UE_LOGF(LogBuoyancySubsystem, Warning, "Buoyancy Subsystem Allocated Bytes: %d", AllocatedSize);
		}
	}));
#endif // WITH_BUOYANCY_MEMORY_TRACKING

//
// Buoyancy Subsystem
//

bool UBuoyancySubsystem::SetEnabled(const bool bEnabled)
{
	if (bEnabled != IsEnabled())
	{
		if (bEnabled)
		{
			CreateSimCallback();
		}
		else
		{
			DestroySimCallback();
		}

		return IsEnabled() == bEnabled;
	}

	// Already had whatever setting
	return true;
}

bool UBuoyancySubsystem::IsEnabled() const
{
	return SimCallback != nullptr;
}

bool UBuoyancySubsystem::SetEnabledWithUpdatedNetModeCallback(const bool bEnabled)
{
	bool bEnabledResult = SetEnabled(bEnabled);

	if (IsEnabled())
	{
		if (SimCallback)
		{
			if (FBuoyancySubsystemSimCallbackInput* AsyncInput = SimCallback->GetProducerInputData_External())
			{
				AsyncInput->NetMode = NetMode;
			}
		}
	}

	return bEnabledResult;
}

#if WITH_BUOYANCY_MEMORY_TRACKING
SIZE_T UBuoyancySubsystem::GetAllocatedSize() const
{
	return SimCallback_AllocatedSize;
}
#endif

void UBuoyancySubsystem::CreateSimCallback()
{
	// Sometimes in PIE, sim callbacks have not been freed at this point.
	// I think it is an issue with world ticking subsystem Deinitialize().
	DestroySimCallback();

	// Create sim callback
	if (Chaos::FPhysicsSolver* Solver = GetSolver())
	{
		// Create the callback for keeping spline data in sync
		ensureAlwaysMsgf(SplineData == nullptr, TEXT("UBuoyancySubsystem::CreateSimCallback: Creating new FBuoyancyWaterSplineDataManager before releasing previous SplineData"));
		SplineData = Solver->CreateAndRegisterSimCallbackObject_External<FBuoyancyWaterSplineDataManager>();

		// Create the main buoyancy sim callback
		ensureAlwaysMsgf(SimCallback == nullptr, TEXT("UBuoyancySubsystem::CreateSimCallback: Creating new FBuoyancySubsystemSimCallback before releasing previous SimCallback"));
		SimCallback = Solver->CreateAndRegisterSimCallbackObject_External<FBuoyancySubsystemSimCallback>();

		// Give the buoyancy sim callback a reference to the spline data callback,
		// so that it'll have access to per-particle spline data
		if (ensureAlwaysMsgf(SimCallback != nullptr && SplineData != nullptr, TEXT("Either SimCallback or SplineData were not properly initialized in UBuoyancySubsystem::CreateSimCallback!")))
		{
			if (FBuoyancySubsystemSimCallbackInput* AsyncInput = SimCallback->GetProducerInputData_External())
			{
				AsyncInput->SplineData = SplineData;
			}
		}

		// Populate an initial async input so that the sim callbacks have the most up-to-date info
		UpdateAllAsyncInputs();
	}
}

void UBuoyancySubsystem::DestroySimCallback()
{
	// Destroy the main sim callback for buoyancy
	if (SimCallback)
	{
		if (Chaos::FPhysicsSolverBase* Solver = SimCallback->GetSolver())
		{
			Solver->UnregisterAndFreeSimCallbackObject_External(SimCallback);
			SimCallback = nullptr;
		}
	}

	// Destroy the sim callback for keeping spline data in sync
	if (SplineData)
	{
		if (Chaos::FPhysicsSolverBase* Solver = SplineData->GetSolver())
		{
			Solver->UnregisterAndFreeSimCallbackObject_External(SplineData);
			SplineData = nullptr;
		}
	}
}

void UBuoyancySubsystem::PostInitialize()
{
	Super::PostInitialize();

	// Cache pointer to ocean fallback depth CVar (defined in Water module)
	OceanFallbackDepthCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.OceanFallbackDepth"));

	// Apply initial runtime settings
	ApplyRuntimeSettings(GetDefault<UBuoyancyRuntimeSettings>(), EPropertyChangeType::ValueSet);


	// Setup callback for when waterbodies are added/removed
	if (FWaterBodyManager* WaterBodyManager = UWaterSubsystem::GetWaterBodyManager(GetWorld()))
	{
		WaterBodyManager->OnWaterBodyAdded.AddUObject(this, &UBuoyancySubsystem::OnWaterBodyAdded);
		WaterBodyManager->OnWaterBodyRemoved.AddUObject(this, &UBuoyancySubsystem::OnWaterBodyRemoved);
		bWaterObjectsChanged = true;
	}

	// Set up callback for when runtime settings change in editor
#if WITH_EDITOR
	GetDefault<UBuoyancyRuntimeSettings>()->OnSettingsChange.AddUObject(this, &UBuoyancySubsystem::ApplyRuntimeSettings);
#endif //WITH_EDITOR
}

void UBuoyancySubsystem::Deinitialize()
{
	DestroySimCallback();

	Super::Deinitialize();
}

void UBuoyancySubsystem::ApplyRuntimeSettings(const UBuoyancyRuntimeSettings* InSettings, EPropertyChangeType::Type ChangeType)
{
	bBuoyancySettingsChanged = true;

	// Runtime settings presents water density in g/cm^3, but we want it in kg/cm^3
	// so introduce a factor of 10^-3 here.
	BuoyancySettings.WaterDensity = Chaos::GCm3ToKgCm3(InSettings->WaterDensity);
	BuoyancySettings.WaterDrag = InSettings->WaterDrag;
	BuoyancySettings.WaterLift = InSettings->WaterLift;
	BuoyancySettings.WaterCollisionChannel = InSettings->CollisionChannelForWaterObjects;
	BuoyancySettings.bKeepAwake = InSettings->bKeepFloatingObjectsAwake;
	BuoyancySettings.MaxNumBoundsSubdivisions = InSettings->MaxNumBoundsSubdivisions;
	BuoyancySettings.MinBoundsSubdivisionVol = InSettings->MinBoundsSubdivisionVol;
	BuoyancySettings.MinVelocityForSurfaceTouchCallback = InSettings->MinVelocityForSurfaceTouchCallback;
	BuoyancySettings.bSplineKeyCacheGrid = InSettings->bEnableSplineKeyCacheGrid;
	BuoyancySettings.SplineKeyCacheGridSize = InSettings->SplineKeyCacheGridSize; 
	BuoyancySettings.SplineKeyCacheLimit = InSettings->SplineKeyCacheLimit;

	// Based on server/client/editor, determine if we should generate callbacks.
	// If we're editor, always generate callbacks. If we're not editor, only
	// generate callbacks on client.	
#if WITH_EDITOR
	BuoyancySettings.SurfaceTouchCallbackFlags = InSettings->SurfaceTouchCallbackFlags;
#else
	UWorld* World = GetWorld();
	BuoyancySettings.SurfaceTouchCallbackFlags
		= (World && World->IsNetMode(NM_Client))
		? InSettings->SurfaceTouchCallbackFlags
		: EBuoyancyEventFlags::None;
#endif

	// Enable or disable
	SetEnabled(InSettings->bBuoyancyEnabled);
}

void UBuoyancySubsystem::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_Tick)

	//
	// The entire point of this tick is to send runtime data to
	// the physics thread which might have changed, and to process
	// outputs which may effect water bodies or result in
	// callbacks.
	//

	Super::Tick(DeltaTime);

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

	if (SimCallback == nullptr)
	{
		return;
	}

	// Update spline info for all water bodies & internal arrays of water objects
	if (bWaterObjectsChanged)
	{
		UpdateSplineData();
	}

	// Only bother sending new async inputs if our buoyancy settings actually changed
	if (bBuoyancySettingsChanged)
	{
		UpdateBuoyancySettings();
	}

	// Send wave reference time to physics thread every frame
	UpdateWaveReferenceTime();

	if (NetMode != World->GetNetMode())
	{
		NetMode = World->GetNetMode();
		UpdateNetMode();
	}

	// Process surface-touched callbacks
	if (BuoyancySettings.SurfaceTouchCallbackFlags != 0)
	{
		ProcessSurfaceTouchCallbacks();
	}
}

void UBuoyancySubsystem::UpdateAllAsyncInputs()
{
	UpdateNetMode();
	UpdateSplineData();
	UpdateBuoyancySettings();
	UpdateWaveReferenceTime();
}

void UBuoyancySubsystem::UpdateNetMode()
{
	// Send net mode to PT
	if (SimCallback)
	{
		if (FBuoyancySubsystemSimCallbackInput* AsyncInput = SimCallback->GetProducerInputData_External())
		{
			AsyncInput->NetMode = NetMode;
		}
	}
}

bool UBuoyancySubsystem::QueryWaterBody(const FVector& InputPosition, const TSharedPtr<FBuoyancyWaterSplineData> WaterData,
	FVector& WaterVel, FVector& WaterPlaneN, FVector& WaterPlanePos)
{		
	float ClosestSplineKey = 0;
	WaterPlanePos = FVector::ZeroVector;
	WaterPlaneN = FVector::ZeroVector;
	WaterVel = FVector::ZeroVector;

	if (SplineData == nullptr || WaterData == nullptr)
	{
		return false;
	}

	if (WaterData->ShouldSampleFromShallowWaterSimulation())
	{
		// eval shallow water - closest point is just the current point adjusted upward in Z		
		float WaterHeight;
		float WaterDepth;
		WaterData->ShallowWaterSimData->SampleShallowWaterSimulationAtPosition(InputPosition, WaterVel, WaterHeight, WaterDepth);
		
		// #todo(dmp): if there is no water here, then return.  This is an approximation that there is water underneath this rigid that might cause interaction
		// ideally, we'd multisample this since we can lose some water interactions this way
		if (WaterDepth > 1e-5)
		{			
			WaterPlanePos = InputPosition;
			WaterPlanePos.Z = WaterHeight;
		
			// compute water plane normal
			WaterPlaneN = WaterData->ShallowWaterSimData->ComputeShallowWaterSimulationNormalAtPosition(InputPosition);

			return true;
		}
	}
	else
	{
		// Find water surface at the nearest point on the spline
		const FVector ParticlePos = InputPosition;

		FVector ClosestPointDerivative = FVector::ZeroVector;
		bool IsInsideSpline = SimCallback->QuerySpline(ParticlePos, *WaterData, ClosestSplineKey, WaterPlanePos, ClosestPointDerivative, WaterPlaneN);

		if (IsInsideSpline)
		{
			WaterVel = WaterData->Velocity->Eval(ClosestSplineKey) * ClosestPointDerivative.GetSafeNormal();

			return true;
		}		
	}

	return false;
}

bool UBuoyancySubsystem::FindOverlappingWaterBodies(const FBox BoundingBox, 
	bool ComputePlane, 
	FVector &ClosestWaterPlaneLocation,
	FVector &ClosestWaterPlaneNormal,
	FVector &ClosestWaterPlaneVelocity,
	TArray<const TSharedPtr<FBuoyancyWaterSplineData>>& CollisionWaterBodySplineData)
{
	check(IsInGameThread());

	if (!SplineData && !ComputePlane)
	{
		return false;
	}

	FCollisionShape BoxShape = FCollisionShape::MakeBox(BoundingBox.GetExtent());

	FCollisionQueryParams QueryParams;
	QueryParams.bTraceComplex = false;
	QueryParams.bReturnPhysicalMaterial = false;			

	FCollisionObjectQueryParams ObjectQueryParams;
	ObjectQueryParams.AddObjectTypesToQuery(BuoyancySettings.WaterCollisionChannel);

	TArray<FOverlapResult> Overlaps;
			
	// perform overlap test to find collisions with the input bounding box
	bool bHit = GetWorld()->OverlapMultiByObjectType(
		Overlaps,
		BoundingBox.GetCenter(),
		FQuat::Identity,
		ObjectQueryParams,
		BoxShape,
		QueryParams
	);

	CollisionWaterBodySplineData.Empty();
				
	bool bFoundWaterBody = false;

	// if we have a hit, find the water bodies we are colliding with
	if (bHit)
	{
		float MinDistance = TNumericLimits <float>::Max();

		for (const FOverlapResult& Result : Overlaps)
		{
			UWaterBodyComponent* WaterBodyComponent = Cast<UWaterBodyComponent>(Result.Component->GetAttachParent());
			if (WaterBodyComponent)
			{	
				if (ComputePlane)
				{
					const TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = 
						WaterBodyComponent->TryQueryWaterInfoClosestToWorldLocation(BoundingBox.GetCenter(),
							EWaterBodyQueryFlags::ComputeLocation
							| EWaterBodyQueryFlags::ComputeNormal
							| EWaterBodyQueryFlags::ComputeVelocity
							| EWaterBodyQueryFlags::ComputeDepth);

					if (!QueryResult.HasValue())
					{
						continue;
					}

					const FWaterBodyQueryResult &WaterCollisionResult = QueryResult.GetValue();

					const FVector WaterPlaneLocation = WaterCollisionResult.GetWaterPlaneLocation();
					const FVector WaterPlaneNormal = WaterCollisionResult.GetWaterPlaneNormal();
					const FVector WaterPlaneVelocity = WaterCollisionResult.GetVelocity();

					const float DistanceToInput = FVector::Distance(WaterPlaneLocation, BoundingBox.GetCenter());

					if (DistanceToInput < MinDistance && WaterCollisionResult.GetWaterPlaneDepth() > SMALL_NUMBER)
					{
						bFoundWaterBody = true;

						MinDistance = DistanceToInput;

						ClosestWaterPlaneLocation = WaterPlaneLocation;
						ClosestWaterPlaneNormal = WaterPlaneNormal;
						ClosestWaterPlaneVelocity = WaterPlaneVelocity;
					}
				}
				else
				{
					for (UPrimitiveComponent* WaterPrimitiveComponent : WaterBodyComponent->GetCollisionComponents(true))
					{
						// Add each object (probably just one) to the objects list
						if (WaterPrimitiveComponent)
						{
							FBodyInstance& BodyInstance = WaterPrimitiveComponent->BodyInstance;
							if (BodyInstance.IsValidBodyInstance())
							{
								Chaos::FSingleParticlePhysicsProxy* WaterProxy = BodyInstance.GetPhysicsActor();
							
								// if we found a valid physics actor, get the PT spline data associated with the particle
								// #todo(dmp): resolve issues with threading since this method is called from the GT but we are
								// querying PT data here that is created from the Buoyancy plugin.
								if (WaterProxy)
								{									
									Chaos::FRigidBodyHandle_External& Body_External = WaterProxy->GetGameThreadAPI();								
								
									const TSharedPtr<FBuoyancyWaterSplineData> *TmpData = SplineData->GetData_PT(WaterProxy->GetGameThreadAPI());								
									if (TmpData)
									{
										bFoundWaterBody = true;
										CollisionWaterBodySplineData.Add(*TmpData);
									}
								}
							}
						}
					}
				}
			}
		}
		return bFoundWaterBody;
	}

	return false;
}

void UBuoyancySubsystem::UpdateSplineData()
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_UpdateWaterBodiesList)

	if (bWaterObjectsChanged == false)
	{
		return;
	}

	if (SplineData == nullptr)
	{
		return;
	}

	if (FWaterBodyManager* WaterBodyManager = UWaterSubsystem::GetWaterBodyManager(GetWorld()))
	{
		// Clear the flag to rebuild water object data
		bWaterObjectsChanged = false;

		// Clear out existing old water object data
		//
		// NOTE: the "false" argument here prevents a resize, since we're just going to
		// repopulate the underlying sparse array again immediately anyway.
		SplineData->ClearData_GT(false);

		// Loop over every registered water body
		WaterBodyManager->ForEachWaterBodyComponent(GetWorld(), [this](UWaterBodyComponent* WaterBodyComponent)
		{
			// Get the metadata object if there is one, and the spline component
			UWaterSplineMetadata* SplineMetadata = WaterBodyComponent->GetWaterSplineMetadata();
			UBakedShallowWaterSimulationComponent *BakedSim = WaterBodyComponent->GetBakedShallowWaterSimulation();
			
			if (const UWaterSplineComponent* SplineComponent = WaterBodyComponent->GetWaterSpline())
			{
				// Copy out water spline data into a shared ptr, to be associated with all
				// child particles and marshaled to PT.
				const Chaos::FRigidTransform3 WaterTransform = WaterBodyComponent->GetComponentTransform();
				const TOptional<FInterpCurveFloat> EmptyOptionalFloat;				

				// Build wave data if this water body has Gerstner waves
				TSharedPtr<FBuoyancyWaveData> WaveData;
				if (bBuoyancyUseWaves && WaterBodyComponent->HasWaves())
				{
					if (UWaterWavesBase* WavesBase = WaterBodyComponent->GetWaterWaves())
					{
						if (const UGerstnerWaterWaves* GerstnerWaves = Cast<UGerstnerWaterWaves>(WavesBase->GetWaterWaves()))
						{
							WaveData = MakeShared<FBuoyancyGerstnerWaveData>(
								GerstnerWaves->GetGerstnerWaves(),
								GerstnerWaves->GetMaxWaveHeight(),
								WaterBodyComponent->TargetWaveMaskDepth);
						}
					}
				}

				const float OceanFallbackDepth = OceanFallbackDepthCVar ? OceanFallbackDepthCVar->GetFloat() : 0.f;

				TSharedPtr<FBuoyancyWaterSplineData> WaterSplineData = MakeShared<FBuoyancyWaterSplineData>(
					WaterTransform,
					SplineComponent->GetSplinePointsPosition(),
					WaterBodyComponent->GetWaterBodyType(),
					SplineMetadata ? SplineMetadata->RiverWidth : EmptyOptionalFloat,
					SplineMetadata ? SplineMetadata->WaterVelocityScalar : EmptyOptionalFloat,
					bUseShallowWaterSimulation && WaterBodyComponent->UseBakedSimulationForQueriesAndPhysics() ? BakedSim->BakedSimulationData : nullptr,
					WaveData,
					SplineMetadata ? SplineMetadata->Depth : EmptyOptionalFloat,
					WaterBodyComponent->GetChannelDepth(),
					WaterBodyComponent->CurveSettings.CurveRampWidth,
					WaterBodyComponent->CurveSettings.ChannelEdgeOffset,
					OceanFallbackDepth
				);

				// Go over each physics object in each primitive component which was generated
				// from this spline, and associate the spline with the particle.
				for (UPrimitiveComponent* WaterPrimitiveComponent : WaterBodyComponent->GetCollisionComponents(true))
				{
					// Add each object (probably just one) to the objects list
					if (WaterPrimitiveComponent)
					{
						FBodyInstance& BodyInstance = WaterPrimitiveComponent->BodyInstance;
						if (BodyInstance.IsValidBodyInstance())
						{
							if (Chaos::FSingleParticlePhysicsProxy* WaterProxy = BodyInstance.GetPhysicsActor())
							{
								SplineData->SetData_GT(WaterProxy->GetGameThreadAPI(), WaterSplineData);
							}
						}
					}
				}
			}
			return true;
		});
	}
}

void UBuoyancySubsystem::UpdateBuoyancySettings()
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_UpdateBuoyancySettings)

	if (SimCallback)
	{
		if (FBuoyancySubsystemSimCallbackInput* AsyncInput = SimCallback->GetProducerInputData_External())
		{
			bBuoyancySettingsChanged = false;
			AsyncInput->BuoyancySettings = MakeUnique<FBuoyancySettings>(BuoyancySettings);
		}
	}
}

void UBuoyancySubsystem::UpdateWaveReferenceTime()
{
	if (SimCallback)
	{
		if (FBuoyancySubsystemSimCallbackInput* AsyncInput = SimCallback->GetProducerInputData_External())
		{
			float Time = 0.f;
			if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld()))
			{
				Time = WaterSubsystem->GetWaterTimeSeconds();
			}
			AsyncInput->WaveReferenceTime = Time;
		}
	}
}

void UBuoyancySubsystem::ProcessSurfaceTouchCallbacks()
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_DispatchCallbacks)

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

	FPhysScene* PhysScene = World->GetPhysicsScene();
	if (PhysScene == nullptr)
	{
		return;
	}

	while (Chaos::TSimCallbackOutputHandle<FBuoyancySubsystemSimCallbackOutput> AsyncOutput = SimCallback->PopFutureOutputData_External())
	{
		// Process surface touches
		for (const FBuoyancySubsystemSimCallbackOutput::FSurfaceTouch& SurfaceTouch : AsyncOutput->SurfaceTouches)
		{
			// Skip if we mask out this touch type
			if ((SurfaceTouch.Flag & BuoyancySettings.SurfaceTouchCallbackFlags) == 0)
			{
				continue;
			}

			// Extract primitive components
			UPrimitiveComponent* WaterComponent = PhysScene->GetOwningComponent<UPrimitiveComponent>(SurfaceTouch.WaterProxy);
			UPrimitiveComponent* RigidComponent = PhysScene->GetOwningComponent<UPrimitiveComponent>(SurfaceTouch.RigidProxy);

			// Skip if either of the components were not valid
			if (WaterComponent == nullptr ||
				RigidComponent == nullptr)
			{
				continue;
			}

			// Get the parental water body component
			AWaterBody* WaterActor = WaterComponent->GetOwner<AWaterBody>();

			const auto DispatchEvent = [&](AActor* Actor)
			{
				// TODO: Actor relevancy check?

				// If the actor implements the event interface, call the surface touched callback
				if (IBuoyancyEventInterface* InterfaceInstance = Cast<IBuoyancyEventInterface>(Actor))
				{
					switch (SurfaceTouch.Flag)
					{
					case EBuoyancyEventFlags::Begin:
						InterfaceInstance->OnSurfaceTouchBegin_Native(
							WaterComponent, RigidComponent,
							SurfaceTouch.Vol, SurfaceTouch.CoM, SurfaceTouch.Vel);
						IBuoyancyEventInterface::Execute_OnSurfaceTouchBegin(
							Actor, WaterActor, WaterComponent, RigidComponent,
							SurfaceTouch.Vol, SurfaceTouch.CoM, SurfaceTouch.Vel);
						break;

					case EBuoyancyEventFlags::Continue:
						InterfaceInstance->OnSurfaceTouching_Native(
							WaterComponent, RigidComponent,
							SurfaceTouch.Vol, SurfaceTouch.CoM, SurfaceTouch.Vel);
						IBuoyancyEventInterface::Execute_OnSurfaceTouching(
							Actor, WaterActor, WaterComponent, RigidComponent,
							SurfaceTouch.Vol, SurfaceTouch.CoM, SurfaceTouch.Vel);
						break;

					case EBuoyancyEventFlags::End:
						InterfaceInstance->OnSurfaceTouchEnd_Native(
							WaterComponent, RigidComponent,
							SurfaceTouch.Vol, SurfaceTouch.CoM, SurfaceTouch.Vel);
						IBuoyancyEventInterface::Execute_OnSurfaceTouchEnd(
							Actor, WaterActor, WaterComponent, RigidComponent,
							SurfaceTouch.Vol, SurfaceTouch.CoM, SurfaceTouch.Vel);
						break;
					}
				}
			};

			DispatchEvent(WaterActor);
			DispatchEvent(RigidComponent->GetOwner());
		}

#if WITH_BUOYANCY_MEMORY_TRACKING
		// Process changes to tracking of memory allocated on physics thread
		if (AsyncOutput->AllocatedSize.IsSet())
		{
			SimCallback_AllocatedSize = *AsyncOutput->AllocatedSize;
		}
#endif
	}
}

TStatId UBuoyancySubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UBuoyancySubsystem, STATGROUP_Tickables);
}

void UBuoyancySubsystem::OnWaterBodyAdded(UWaterBodyComponent* WaterBodyComponent)
{
	bWaterObjectsChanged = true;
}

void UBuoyancySubsystem::OnWaterBodyRemoved(UWaterBodyComponent* WaterBodyComponent)
{
	bWaterObjectsChanged = true;
}

Chaos::FPhysicsSolver* UBuoyancySubsystem::GetSolver() const
{
	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* Scene = World->GetPhysicsScene())
		{
			if (Chaos::FPhysicsSolver* Solver = Scene->GetSolver())
			{
				return Solver;
			}
		}
	}

	return nullptr;
}


//
// Buoyancy Sim Callback
//

void FBuoyancySubsystemSimCallbackInput::Reset()
{
	SplineData.Reset();
	BuoyancySettings.Reset();
	NetMode.Reset();
	WaveReferenceTime.Reset();
}

void FBuoyancySubsystemSimCallbackOutput::Reset()
{
	SurfaceTouches.Reset();

#if WITH_BUOYANCY_MEMORY_TRACKING
	AllocatedSize.Reset();
#endif
}

SIZE_T FBuoyancySubsystemSimCallback::GetAllocatedSize() const
{
	return
		BuoyancyParticleData.GetAllocatedSize() +
		SplineKeyCache.GetAllocatedSize();
}

void FBuoyancySubsystemSimCallback::OnPreSimulate_Internal()
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_OnPreSimulate)

	// If we were sent new buoyancy settings or data, update our local sim copy
	if (const FBuoyancySubsystemSimCallbackInput* Input = GetConsumerInput_Internal())
	{
		if (Input->NetMode.IsSet())
		{
			NetMode = *Input->NetMode;
		}

		if (Input->SplineData.IsSet())
		{
			SplineData = *Input->SplineData;

			// New spline data means we need to reset our spline key cache
			if (!bUseAccurateIntegrationForSplines)
			{
				SplineKeyCache.Reset();
			}

			// New spline data also means that any internal storage of water particles
			// might be out of date. Clear all data that potentially contains references
			// to old water particles.
			//
			// NOTE: This may cause new-submersion events to trigger on already submerged
			// objects when water splines are added/removed, depending on velocity-
			// filtering options. It might be desirable to instead add a mechanism to
			// suppress events for one frame instead, or to remove only the data which
			// relate to water particles which have been removed.
			BuoyancyParticleData.Interactions.Reset();
			BuoyancyParticleData.SubmersionMetaData.Reset();
			BuoyancyParticleData.PrevSubmersionMetaData.Reset();
		}

		if (Input->BuoyancySettings.IsValid())
		{
			BuoyancySettings = MoveTemp(Input->BuoyancySettings);

			// Set key cache properties
			if (!bUseAccurateIntegrationForSplines)
			{
				SplineKeyCache.SetGridSize(BuoyancySettings->SplineKeyCacheGridSize);
				SplineKeyCache.SetCacheLimit(BuoyancySettings->SplineKeyCacheLimit);
			}
		}

		if (Input->WaveReferenceTime.IsSet())
		{
			WaveReferenceTime = *Input->WaveReferenceTime;
		}
	}

	// If we don't have a valid buoyancy settings object, don't continue
	if (BuoyancySettings.IsValid() == false)
	{
		return;
	}

	// If we have debug draw enabled
#if CHAOS_DEBUG_DRAW
	if (bBuoyancyDebugDraw)
	{
		// If we're using spline key cache grid, debug draw all cached points
		if (BuoyancySettings->bSplineKeyCacheGrid && !bUseAccurateIntegrationForSplines)
		{
			SplineKeyCache.ForEachSplineKey([this](const FBuoyancyWaterSplineData& WaterSpline, const FVector& LocalPos, float SplineKey)
			{
				// Draw a box representing this spline's grid cell
				const FVector WorldPos = WaterSpline.Transform.TransformPosition(LocalPos);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(WorldPos, FVector(SplineKeyCache.GetGridSize() * .5f), FQuat::Identity, FColor::White, false, -1.f, -1, 1.f);

				// Draw an arrow from the center to the closest
				const FVector ClosestPointLocal = WaterSpline.Position.Eval(SplineKey);
				const FVector ClosestPoint = WaterSpline.Transform.TransformPosition(ClosestPointLocal);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(WorldPos, ClosestPoint, 20.f, FColor::Silver, false, -1.f, -1, 1.f);
			});
		}
	}
#endif
}

void FBuoyancySubsystemSimCallback::OnMidPhaseModification_Internal(Chaos::FMidPhaseModifierAccessor& MidPhaseAccessor)
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_OnMidPhaseModification)

	// Don't do any processing until we know what machine we're on
	if (NetMode == ENetMode::NM_MAX)
	{
		return;
	}

	// If we don't have a spline data manager, early out
	if (SplineData == nullptr)
	{
		return;
	}

	// If we don't have a valid buoyancy settings object, don't continue
	if (BuoyancySettings.IsValid() == false)
	{
		return;
	}

	// Get the solver
	Chaos::FPhysicsSolverBase* SolverBase = GetSolver();
	if (SolverBase == nullptr)
	{
		return;
	}
	// Why does cast-checked return a ref? That makes me think
	// it's not actually doing a check...
	Chaos::FPBDRigidsSolver& PBDSolver = SolverBase->CastChecked();

	// Get the evolution
	Chaos::FPBDRigidsEvolution* Evolution = PBDSolver.GetEvolution();
	if (Evolution == nullptr)
	{
		return;
	}

	// SparseArray implements move semantics, so these swaps should amount to pointer swaps.
	// This way array memories stick around even when reset/swapped so we don't do many
	// new allocations.
	Swap(BuoyancyParticleData.Submersions, BuoyancyParticleData.PrevSubmersions);
	Swap(BuoyancyParticleData.SubmersionMetaData, BuoyancyParticleData.PrevSubmersionMetaData);

	// Clear arrays for the following phases, but keep their memory allocated
	BuoyancyParticleData.Interactions.Reset();
	BuoyancyParticleData.Submersions.Reset();
	BuoyancyParticleData.SubmergedShapes.Reset();
	BuoyancyParticleData.SubmersionMetaData.Reset();

	// Build list of "submersions"
	TrackInteractions(PBDSolver, *Evolution, MidPhaseAccessor);

	// Process the list of interactions that we built from the midphases
	ProcessInteractions(*Evolution);

	// Apply buoyant forces resulting from submersions
	ApplyBuoyantForces(*Evolution);
	
	// Generate async outputs for callback data
	GenerateCallbackData();

#if WITH_BUOYANCY_MEMORY_TRACKING
	// Generate async outputs for allocation data
	GenerateAllocationData();
#endif
}

void FBuoyancySubsystemSimCallback::TrackInteractions(
	Chaos::FPBDRigidsSolver& PBDSolver,
	Chaos::FPBDRigidsEvolution& Evolution,
	Chaos::FMidPhaseModifierAccessor& MidPhaseAccessor)
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_VisitMidphases)

	// The spline userdatapt manager contains a PT array of all water splines indexed on
	// water particle unique index. Therefore, we can use it to loop over all water body
	// particles and check their midphases.
	SplineData->VisitData_PT([this, &PBDSolver, &Evolution, &MidPhaseAccessor](Chaos::FUniqueIdx ParticleIdx, const TSharedPtr<FBuoyancyWaterSplineData>& WaterData)
	{
		// If the WaterData shared ptr is invalid, skip this one
		if (WaterData.IsValid() == false)
		{
			return;
		}

		// Get water body particle proxy from index
		Chaos::FSingleParticlePhysicsProxy* ParticleProxy = PBDSolver.GetParticleProxy_PT(ParticleIdx);
		if (ParticleProxy == nullptr || ParticleProxy->GetMarkedDeleted())
		{
			return;
		}

		// Get water body particle handle from proxy
		Chaos::FGeometryParticleHandle* WaterParticle = ParticleProxy->GetHandle_LowLevel();
		if (WaterParticle == nullptr)
		{
			return;
		}

		// Get all midphases which involve this particle
		Chaos::FMidPhaseModifierParticleRange MidPhases = MidPhaseAccessor.GetMidPhases(WaterParticle);

		// Loop over all midphases
		for (auto& MidPhase : MidPhases)
		{
			// Make sure we have two valid particles
			Chaos::FGeometryParticleHandle* OtherParticle = MidPhase.GetOtherParticle(WaterParticle);
			if (WaterParticle == nullptr ||
				OtherParticle == nullptr)
			{
				continue;
			}

			// Make sure the non-water particle is backed by a rigid
			Chaos::FPBDRigidParticleHandle* RigidParticle = OtherParticle->CastToRigidParticle();
			if (RigidParticle == nullptr)
			{
				continue;
			}

			// Finally... we know for sure this is a midphase that we wanna process
			TrackInteraction(Evolution, WaterParticle, RigidParticle, *WaterData.Get(), MidPhase);
		}
	});
}

bool FBuoyancySubsystemSimCallback::QuerySpline(
	const FVector &QueryPos, 
	const FBuoyancyWaterSplineData& WaterSpline,
	float &ClosestSplineKey,
	FVector &ClosestPoint,
	FVector &ClosestPointDerivative,
	FVector &WaterN)
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_SplineEvaluation)

	ClosestSplineKey = 0;
	ClosestPoint = FVector::ZeroVector;
	WaterN = FVector::ZeroVector;	

	// Find water surface at the nearest point on the spline	
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_SplineEvaluation_FindNearest)

		if (BuoyancySettings->bSplineKeyCacheGrid)
		{
			ClosestSplineKey = SplineKeyCache.GetClosestSplineKey(WaterSpline, QueryPos);
		}
		else
		{
			const FVector ParticleLocalPos = WaterSpline.Transform.InverseTransformPosition(QueryPos);
			float ParticleDistance;
			ClosestSplineKey = WaterSpline.Position.FindNearest(ParticleLocalPos, ParticleDistance);
		}
	}
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_SplineEvaluation_Eval)
		ClosestPoint = WaterSpline.Transform.TransformPosition(WaterSpline.Position.Eval(ClosestSplineKey));
	}

	// Get water surface level and normal
	// Don't build interaction if outside of water body
	{			
		// Find water surface at the nearest point on the spline			
		ClosestPointDerivative = WaterSpline.Transform.TransformVector(
		WaterSpline.Position.EvalDerivative(ClosestSplineKey));

		// Water normal direction depends on body type
		if (WaterSpline.BodyType == EWaterBodyType::River)
		{
			// River water normal can be determined by the relationship of the 
			// derivative of the spline position to the up vector.
			//
			// NOTE: This calculation breaks down in the limit of purely
			// vertical water
			const FVector SplineRight = FVector::CrossProduct(FVector::UpVector, ClosestPointDerivative);
			const FVector SplineUp = FVector::CrossProduct(ClosestPointDerivative, SplineRight);
			WaterN = SplineUp.GetSafeNormal();
		}
		else
		{
			WaterN = FVector::UpVector;
		}

		// Project the position difference onto the water surface
		const FVector Diff = ClosestPoint - QueryPos;
		const FVector LateralDiff = Diff - (WaterN * FVector::DotProduct(WaterN, Diff));

		// Different water body types have different ways of determining
		// whether a point is laterally inside their volume.
		switch (WaterSpline.BodyType)
		{
			case EWaterBodyType::River:
			{
				if (WaterSpline.Width.IsSet())
				{
					// If distance to spline is greater than the width of the spline,
					// then this is a river and we're outside of it.
					const float Width = WaterSpline.Width->Eval(ClosestSplineKey);
					const float DistSq = FVector::DotProduct(LateralDiff, LateralDiff);
					const float WidthSq = Width * Width * .25f;
					if (DistSq > WidthSq) { return false; }
				}
				break;
			}

			case EWaterBodyType::Lake:
			{
				// Determine if we're inside the lake by projecting the horizontal spline
				// diff onto the cross product of the spline direction and the up-vector
				// (ie, the right-vector)
				const FVector RightVector = FVector::CrossProduct(ClosestPointDerivative, FVector::UpVector);
				const float DiffProj = FVector::DotProduct(RightVector, LateralDiff);
				if (DiffProj < SMALL_NUMBER) { return false; }
				break;
			}

			case EWaterBodyType::Ocean:
			{
				// Determine if we're inside the ocean by projecting the horizontal spline
				// diff onto the cross product of the spline direction and the up-vector
				// (ie, the right-vector).  Note this is reversed from lakes since the splines
				// define the carved out area that is land.
				const FVector RightVector = FVector::CrossProduct(ClosestPointDerivative, FVector::UpVector);
				const float DiffProj = FVector::DotProduct(RightVector, LateralDiff);
				if (DiffProj > SMALL_NUMBER) { return false; }
				break;
			}
		}
			
#if CHAOS_DEBUG_DRAW
		if (bBuoyancyDebugDraw && !bUseAccurateIntegrationForSplines)
		{
			// Spline Color
			const FColor SplineColor = FColor::Cyan;

			// Draw projection onto the line
			const FVector SurfacePoint = ClosestPoint - LateralDiff;
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(QueryPos, SurfacePoint, SplineColor, false, -1.f, -1, 6.f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(SurfacePoint, ClosestPoint, SplineColor, false, -1.f, -1, 3.f);

			// Draw a section of the spline near the spline key
			Chaos::FVec3 PrevPoint;
			bool bFirst = true;
			for (float SplineKey = ClosestSplineKey - .1f; SplineKey <= ClosestSplineKey + .1f; SplineKey += .05f)
			{
				const FVector SplinePoint = WaterSpline.Transform.TransformPosition(WaterSpline.Position.Eval(SplineKey));
				if (bFirst)
				{
					bFirst = false;
				}
				else
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(PrevPoint, SplinePoint, 15.f, SplineColor, false, -1.f, -1, 3.f);
				}
				PrevPoint = SplinePoint;
			}

			// Draw water surface normal at the surface point
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(SurfacePoint, SurfacePoint + (WaterN * 100.f), 20.f, FColor::Green, false, -1.f, -1, 3.f);
		}
#endif
	}

	return true;
}

void FBuoyancySubsystemSimCallback::TrackInteraction(
	Chaos::FPBDRigidsEvolution& Evolution,
	Chaos::FGeometryParticleHandle* WaterParticle,
	Chaos::FPBDRigidParticleHandle* RigidParticle,
	const FBuoyancyWaterSplineData& WaterSpline,
	Chaos::FMidPhaseModifier& MidPhase)
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_TrackInteraction)

	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_DisableMidPhase)

		// Always disable midphases with water
		MidPhase.Disable();
	}
	
	TSharedPtr<FBuoyancyWaterSampler> WaterSampler = nullptr;

	if (WaterSpline.ShouldSampleFromShallowWaterSimulation())
	{
		// eval shallow water - closest point is just the current point adjusted upward in Z
		const FVector ParticlePos = RigidParticle->XCom();
		FVector WaterVelocity;
		float WaterHeight;
		float WaterDepth;
		WaterSpline.ShallowWaterSimData->SampleShallowWaterSimulationAtPosition(ParticlePos, WaterVelocity, WaterHeight, WaterDepth);
		
		// #todo(dmp): if there is no water here, then return.  This is an approximation that there is water underneath this rigid that might cause interaction
		// ideally, we'd multisample this since we can lose some water interactions this way
		if (WaterDepth < 1e-5)
		{
			return;
		}

		FVector ClosestWaterPointToParticle = ParticlePos;
		ClosestWaterPointToParticle.Z = WaterHeight;
		
		const TWeakObjectPtr<UShallowWaterSimulationDataBase> ShallowWaterSimData = WaterSpline.ShallowWaterSimData;

		WaterSampler = TSharedPtr<FBuoyancyWaterSampler>(new FBuoyancyShallowWaterSampler(ShallowWaterSimData, ClosestWaterPointToParticle, WaterVelocity));
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_SplineEvaluation)

		float ClosestSplineKey = 0;
		FVector ClosestPoint = FVector::ZeroVector;
		FVector ClosestPointDerivative = FVector::ZeroVector;
		FVector WaterN = FVector::ZeroVector;
		
		// Find water surface at the nearest point on the spline
		const FVector ParticlePos = RigidParticle->XCom();
		bool IsInslideSpline = QuerySpline(ParticlePos, WaterSpline, ClosestSplineKey, ClosestPoint, ClosestPointDerivative, WaterN);
		if (!IsInslideSpline)
		{
			return;
		}		
		
		if (bBuoyancyUseWaves && WaterSpline.HasWaves())
		{
			WaterSampler = TSharedPtr<FBuoyancyWaterSampler>(new FBuoyancyConstantSplineWithWavesSampler(
				WaterSpline, ParticlePos, ClosestPoint, ClosestPointDerivative, ClosestSplineKey, WaterN, WaveReferenceTime, bBuoyancyUseSimpleWaves));

#if CHAOS_DEBUG_DRAW
			if (bBuoyancyDebugDraw)
			{
				auto DerivedSampler = StaticCastSharedPtr<FBuoyancyConstantSplineWithWavesSampler>(WaterSampler);

				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ParticlePos, ClosestPoint, 20, FColor::Purple, false, -1.f, -1, 2.f);
				
				FVector BottomPoint = ParticlePos;
				FVector Vel;
				float H, D;
				DerivedSampler->SampleWaterAtPosition(ParticlePos, Vel, H, D);
				BottomPoint.Z = ClosestPoint.Z - D;
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ParticlePos, BottomPoint, 20, FColor::Purple, false, -1.f, -1, 2.f);
			}
#endif
		}
		else
		{
		WaterSampler = TSharedPtr<FBuoyancyWaterSampler>(new FBuoyancyConstantSplineSampler(WaterSpline, ClosestPoint, ClosestPointDerivative, ClosestSplineKey, WaterN));				
	}
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_AddInteraction)

		// If this particle already has a list of interactions, add to it. Otherwise
		// make a new one.		
		BuoyancyParticleData.GetInteractions(*RigidParticle).Add({
			RigidParticle,
			WaterParticle,
			WaterSampler
		});
	}
}

void FBuoyancySubsystemSimCallback::ProcessInteractions(Chaos::FPBDRigidsEvolution& Evolution)
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_ProcessInteractions)

	for (FBuoyancyInteractionArray& ParticleInteractions : BuoyancyParticleData.Interactions)
	{
		// Sort this particle's water interactions by water level - highest to lowest
		ParticleInteractions.Sort([](const FBuoyancyInteraction& A, const FBuoyancyInteraction& B)
		{
			return A.WaterSampler->GetClosestWaterPointToParticle().Z > B.WaterSampler->GetClosestWaterPointToParticle().Z;
		});

		// Process each interaction
		for (FBuoyancyInteraction& Interaction : ParticleInteractions)
		{
			if (!Interaction.WaterSampler.IsValid())
			{
				UE_LOGF(LogBuoyancySubsystem, Warning, "Skipped invalid buoyancy interaction");
				return;
			}
			else if (Interaction.WaterSampler->GetSamplerType() == EWaterSamplerType::ShallowWater
				|| Interaction.WaterSampler->GetSamplerType() == EWaterSamplerType::ConstantSplineWithWaves
				|| bUseAccurateIntegrationForSplines)
			{
				ProcessAccurateInteraction(Evolution, Interaction, Interaction.WaterSampler);
			}
			else
			{
				ProcessInteraction(Evolution, Interaction);
			}
		}
	}
}

void FBuoyancySubsystemSimCallback::ProcessAccurateInteraction(Chaos::FPBDRigidsEvolution& Evolution, FBuoyancyInteraction& Interaction, TSharedPtr<FBuoyancyWaterSampler> WaterSampler)
{		
	const Chaos::FReal DeltaSeconds = GetDeltaTime_Internal();	

	float SubmergedVol = 0;
	Chaos::FVec3 SubmergedCoM = Chaos::FVec3::ZeroVector;
	Chaos::FVec3 DragForce = Chaos::FVec3::ZeroVector;
	Chaos::FVec3 DragTorque = Chaos::FVec3::ZeroVector;
	Chaos::FVec3 BuoyancyForce = Chaos::FVec3::ZeroVector;
	Chaos::FVec3 BuoyancyTorque = Chaos::FVec3::ZeroVector;
	float TotalParticleVol = 0;

	if (!Interaction.WaterSampler.IsValid())
	{
		UE_LOGF(LogBuoyancySubsystem, Warning, "Skipped invalid buoyancy interaction ");
		return;
	}
	else if (Interaction.WaterSampler->GetSamplerType() == EWaterSamplerType::ConstantSpline)
	{
		BuoyancyAlgorithms::ComputeSubmergedVolumeAndForcesForParticle(BuoyancyParticleData,
			Interaction.RigidParticle, Interaction.WaterParticle,
			StaticCastSharedPtr<FBuoyancyConstantSplineSampler>(Interaction.WaterSampler),
			Evolution, DeltaSeconds, BuoyancySettings->WaterDensity, BuoyancySettings->WaterDrag, BuoyancySettings->WaterLift,
			TotalParticleVol, SubmergedVol, SubmergedCoM, DragForce, DragTorque, BuoyancyForce, BuoyancyTorque);
	}
	else if (Interaction.WaterSampler->GetSamplerType() == EWaterSamplerType::ConstantSplineWithWaves)
	{
		BuoyancyAlgorithms::ComputeSubmergedVolumeAndForcesForParticle(BuoyancyParticleData,
			Interaction.RigidParticle, Interaction.WaterParticle,
			StaticCastSharedPtr<FBuoyancyConstantSplineWithWavesSampler>(Interaction.WaterSampler),
			Evolution, DeltaSeconds, BuoyancySettings->WaterDensity, BuoyancySettings->WaterDrag, BuoyancySettings->WaterLift,
			TotalParticleVol, SubmergedVol, SubmergedCoM, DragForce, DragTorque, BuoyancyForce, BuoyancyTorque);
	}
	else if (Interaction.WaterSampler->GetSamplerType() == EWaterSamplerType::ShallowWater)
	{
		BuoyancyAlgorithms::ComputeSubmergedVolumeAndForcesForParticle(BuoyancyParticleData,
			Interaction.RigidParticle, Interaction.WaterParticle,
			StaticCastSharedPtr<FBuoyancyShallowWaterSampler>(Interaction.WaterSampler),
			Evolution, DeltaSeconds, BuoyancySettings->WaterDensity, BuoyancySettings->WaterDrag, BuoyancySettings->WaterLift,
			TotalParticleVol, SubmergedVol, SubmergedCoM, DragForce, DragTorque, BuoyancyForce, BuoyancyTorque);
	}

	// Apply forces to the particle
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_ShallowWaterIntegrateForces)

		Chaos::FConstGenericParticleHandle RigidGeneric(Interaction.RigidParticle);

#if CHAOS_DEBUG_DRAW
		if (bBuoyancyDebugDraw)
		{
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(SubmergedCoM, 20, 10, FColor::Green, false, -1.f, -1, 2.0f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(RigidGeneric->PCom(), 20, 10, FColor::Blue, false, -1.f, -1, 2.0f);

			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(RigidGeneric->PCom(), RigidGeneric->PCom() + DragForce + BuoyancyForce, 20, FColor::Blue, false, -1.f, -1, 2.f);
		}
#endif

		// #todo(dmp): if we do this integration to accelerations per shape, then each shape can have different densities per shape.
		// This will change to integrate the accumulated accelerations instead of forces

		const Chaos::FReal InvM = RigidGeneric->InvM();
		const Chaos::FMatrix33 WorldInvI = Chaos::Utilities::ComputeWorldSpaceInertia(RigidGeneric->RCom(), RigidGeneric->ConditionedInvI());

		// Buoyancy - explicit integration (position-dependent, not velocity-dependent)
		const Chaos::FVec3 WorldForce = DragForce + BuoyancyForce;
		const Chaos::FVec3 WorldTorque = DragTorque + BuoyancyTorque;

		Chaos::FVec3 DeltaV = InvM * BuoyancyForce * DeltaSeconds;
		Chaos::FVec3 DeltaW = (WorldInvI * BuoyancyTorque) * DeltaSeconds;

		if (bBuoyancyUseImplicitDrag)
		{
			// Drag - semi-implicit integration to prevent overshoot.
			// Quadratic drag (F ~ v^2) with explicit Euler can overshoot the relative
			// velocity and reverse motion direction. Semi-implicit divides by
			// (1 + beta*dt) where beta is the effective drag coefficient,
			// which is unconditionally stable.
			// Linear and angular are treated independently since drag torque
			// depends on both v and omega (per-triangle v_local = v_cm + omega x r).
			Chaos::FVec3 DeltaVDrag = InvM * DragForce * DeltaSeconds;
			Chaos::FVec3 DeltaWDrag = (WorldInvI * DragTorque) * DeltaSeconds;

			// Linear: project drag delta-V onto relative velocity direction
			const Chaos::FVec3 RelativeV = WaterSampler->GetAverageWaterVelocity() - Interaction.RigidParticle->GetV();
			const float RelVMag = RelativeV.Size();
			if (RelVMag > UE_SMALL_NUMBER)
			{
				const Chaos::FVec3 RelVDir = RelativeV / RelVMag;
				const float DragAlongRelV = Chaos::FVec3::DotProduct(DeltaVDrag, RelVDir);

				if (DragAlongRelV > UE_SMALL_NUMBER)
				{
					const float ImplicitScale = 1.f / (1.f + DragAlongRelV / RelVMag);
					DeltaVDrag *= ImplicitScale;
				}
			}

			// Angular: project drag delta-W onto angular velocity direction
			// (water has no angular velocity, so omega itself is the "relative" angular velocity)
			const Chaos::FVec3 W = Interaction.RigidParticle->GetW();
			const float WMag = W.Size();
			if (WMag > UE_SMALL_NUMBER)
			{
				const Chaos::FVec3 WDir = W / WMag;
				// Negate because drag torque opposes angular velocity (DeltaWDrag points opposite to W),
				// unlike the linear case where DeltaVDrag aligns with RelativeV.
				const float DragAlongW = -Chaos::FVec3::DotProduct(DeltaWDrag, WDir);

				if (DragAlongW > UE_SMALL_NUMBER)
				{
					const float ImplicitScaleW = 1.f / (1.f + DragAlongW / WMag);
					DeltaWDrag *= ImplicitScaleW;
				}
			}

			DeltaV += DeltaVDrag;
			DeltaW += DeltaWDrag;
		}
		else
		{
			// Explicit Euler for all forces (original behavior)
			DeltaV = InvM * WorldForce * DeltaSeconds;
			DeltaW = (WorldInvI * WorldTorque) * DeltaSeconds;
		}
		
		if (SubmergedVol > SMALL_NUMBER)
		{
			// Get the submersion for this particle. If it's new, the particle ptr will be null.
			// If it's not new, make sure the particle ptr is the right particle.
			FBuoyancySubmersion& Submersion = BuoyancyParticleData.GetSubmersion(*Interaction.RigidParticle);
			if (Submersion.Particle != nullptr)
			{
				ensureMsgf(Submersion.Particle == Interaction.RigidParticle, TEXT("Something went wrong - there's a particle index mismatch in the Submersions sparse array"));
			}

			Submersion.Particle = Interaction.RigidParticle;

			Chaos::FSingleParticlePhysicsProxy* RigidProxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(Submersion.Particle->PhysicsProxy());
			Submersion.SyncTimestamp = RigidProxy->GetSyncTimestamp();

			// Get the weighted-average CoM
			// NOTE: The unchecked division should be safe since we already
			// know SubmergedVol > SMALL_NUMBER
			const float TotalSubmergedVol = Submersion.Vol + SubmergedVol;
			Submersion.CoM = ((Submersion.CoM * Submersion.Vol) + (SubmergedCoM * SubmergedVol)) / TotalSubmergedVol;

			// Sum the volumes
			Submersion.Vol = TotalSubmergedVol;

			// Get the weighted-average slerped water surface norm
			const float VolRatio = SubmergedVol / TotalSubmergedVol;
						
			// #todo(dmp): Split out submersion since we don't need norm here- only deltav and deltaw
			// #todo(dmp): output the perimeter of intersection
			Submersion.Vel = FMath::Lerp(Submersion.Vel, WaterSampler->GetAverageWaterVelocity(), VolRatio);
			
			Submersion.HasIntegratedQuantities = true;
			Submersion.DeltaV = FMath::Lerp(Submersion.DeltaV, DeltaV, VolRatio);
			Submersion.DeltaW = FMath::Lerp(Submersion.DeltaW, DeltaW, VolRatio);

			// potentially add a surface callback
			AddSurfaceTouchCallback(Interaction, Submersion, TotalParticleVol, SubmergedVol, SubmergedCoM);
		}
	}
}

void FBuoyancySubsystemSimCallback::ProcessInteraction(Chaos::FPBDRigidsEvolution& Evolution, FBuoyancyInteraction& Interaction)
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_ProcessInteraction)

	TSharedPtr<FBuoyancyConstantSplineSampler> ConstantSplineSampler;
	if (!Interaction.WaterSampler.IsValid())
	{
		UE_LOGF(LogBuoyancySubsystem, Warning, "Skipped invalid buoyancy interaction ");
		return;
	}
	else if (Interaction.WaterSampler->GetSamplerType() == EWaterSamplerType::ConstantSpline)
	{
		ConstantSplineSampler = StaticCastSharedPtr<FBuoyancyConstantSplineSampler>(Interaction.WaterSampler);		
	}
	else
	{
		UE_LOGF(LogBuoyancySubsystem, Warning, "Skipped buoyancy interaction that isn't based on water splines");
		return;
	}
	
	// Compute submerged volume and CoM
	float SubmergedVol;
	Chaos::FVec3 SubmergedCoM;
	float TotalVol;
	if (BuoyancyAlgorithms::ComputeSubmergedVolume(BuoyancyParticleData, Evolution, Interaction.RigidParticle, Interaction.WaterParticle, ConstantSplineSampler->GetClosestWaterPointToParticle(), ConstantSplineSampler->WaterN, BuoyancySettings->MaxNumBoundsSubdivisions, BuoyancySettings->MinBoundsSubdivisionVol, SubmergedVol, SubmergedCoM, TotalVol))
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_BuildSubmersions)

		// If any volume was submerged, take the weighted average of the centers of mass to get the
		// approximate submerged center of mass, and apply the buoyancy force there.
		if (SubmergedVol > SMALL_NUMBER)
		{
			// sample water spline for velocity with this call
			ConstantSplineSampler->InitializeForSubmersion();

			// Get the submersion for this particle. If it's new, the particle ptr will be null.
			// If it's not new, make sure the particle ptr is the right particle.
			FBuoyancySubmersion& Submersion = BuoyancyParticleData.GetSubmersion(*Interaction.RigidParticle);
			if (Submersion.Particle != nullptr)
			{
				ensureMsgf(Submersion.Particle == Interaction.RigidParticle, TEXT("Something went wrong - there's a particle index mismatch in the Submersions sparse array"));
			}

			Submersion.Particle = Interaction.RigidParticle;

			Chaos::FSingleParticlePhysicsProxy* RigidProxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(Submersion.Particle->PhysicsProxy());
			Submersion.SyncTimestamp = RigidProxy->GetSyncTimestamp();

			// Get the weighted-average CoM
			// NOTE: The unchecked division should be safe since we already
			// know SubmergedVol > SMALL_NUMBER
			const float TotalSubmergedVol = Submersion.Vol + SubmergedVol;
			Submersion.CoM = ((Submersion.CoM * Submersion.Vol) + (SubmergedCoM * SubmergedVol)) / TotalSubmergedVol;

			// Sum the volumes
			Submersion.Vol = TotalSubmergedVol;

			// Get the weighted-average slerped water surface norm
			const float VolRatio = SubmergedVol / TotalSubmergedVol;
			const FQuat NormRot = FQuat::FindBetweenNormals(Submersion.Norm, ConstantSplineSampler->WaterN);
			Submersion.Norm = NormRot.Slerp(FQuat::Identity, NormRot, VolRatio).GetUpVector();

			// Blend water velocities
			Submersion.Vel = FMath::Lerp(Submersion.Vel, ConstantSplineSampler->WaterVel, VolRatio);

			// potentially add a surface callback
			AddSurfaceTouchCallback(Interaction, Submersion, TotalVol, SubmergedVol, SubmergedCoM);
		}
	}
}

void FBuoyancySubsystemSimCallback::AddSurfaceTouchCallback(FBuoyancyInteraction& Interaction, FBuoyancySubmersion& Submersion, float TotalVol, float SubmergedVol, FVector SubmergedCoM)
{
	// If this is a surface touch record it for callback, if 
	if (BuoyancySettings->SurfaceTouchCallbackFlags != 0 &&
		TotalVol > SubmergedVol * (1.f + UE_KINDA_SMALL_NUMBER))
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_BuildSubmersionCallbackData)

			// Proceed only if this CoM is moving fast enough to generate events
			const Chaos::FVec3 CoMDiff = Submersion.CoM - Interaction.RigidParticle->XCom();
		const Chaos::FVec3 CoMVel = Interaction.RigidParticle->GetV() + Chaos::FVec3::CrossProduct(Interaction.RigidParticle->GetW(), CoMDiff);
		const float CoMVelSq = Chaos::FVec3::DotProduct(CoMVel, CoMVel);
		const float MinVel = BuoyancySettings->MinVelocityForSurfaceTouchCallback;
		const float MinVelSq = MinVel * MinVel;

		if (CoMVelSq > MinVelSq)
		{
			// If we don't have a metadata for this particle yet, add one
			FBuoyancySubmersionMetaData& MetaData = BuoyancyParticleData.GetSubmersionMetaData(*Interaction.RigidParticle);

			// If we haven't already maxed out on water contacts, add one
			if (MetaData.WaterContacts.Num() < FBuoyancySubmersionMetaData::MaxNumWaterContacts)
			{
				Chaos::FSingleParticlePhysicsProxy* WaterProxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(Interaction.WaterParticle->PhysicsProxy());
				MetaData.WaterContacts.Add({ Interaction.WaterParticle, WaterProxy->GetSyncTimestamp(), SubmergedVol, SubmergedCoM, CoMVel });
			}
		}
	}
}

void FBuoyancySubsystemSimCallback::ApplyBuoyantForces(Chaos::FPBDRigidsEvolution& Evolution)
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_ApplyBuoyantForces)

	// How much time has the sim ticked this frame
	const Chaos::FReal DeltaSeconds = GetDeltaTime_Internal();

	// Get perparticle gravity rule, for figuring out the effective gravity on buoyant objects
	const Chaos::FPerParticleGravity* PerParticleGravity = &Evolution.GetGravityForces();

	// Apply all buoyant forces
	for (const FBuoyancySubmersion& Submersion : BuoyancyParticleData.Submersions)
	{
		Chaos::FVec3 DeltaV;
		Chaos::FVec3 DeltaW;

		bool IsValidForce = true;

		// If the submersion has integrated velocity and torque, use that, otherwise, compute it
		if (Submersion.HasIntegratedQuantities)
		{			
			DeltaV = Submersion.DeltaV;
			DeltaW = Submersion.DeltaW;							
		}
		else
		{
			// Figure out the gravity level of the particle
			const int32 GravityGroupIndex = Submersion.Particle->GravityGroupIndex();
			const Chaos::FVec3 GravityAccel
				= PerParticleGravity != nullptr && GravityGroupIndex != INDEX_NONE
				? (Chaos::FVec3)PerParticleGravity->GetAcceleration(GravityGroupIndex)
				: Chaos::FVec3::DownVector * 980.f; // Default to "regular" gravity

			// Compute delta linear and angular velocities due to buoyancy. If they're big enough to
			// matter, apply them
			IsValidForce = BuoyancyAlgorithms::ComputeBuoyantForce(Submersion.Particle, DeltaSeconds,
				BuoyancySettings->WaterDensity, BuoyancySettings->WaterDrag, GravityAccel, Submersion.CoM, Submersion.Vol, Submersion.Vel, Submersion.Norm, DeltaV, DeltaW);
		}

		if (IsValidForce)
		{
			DeltaV = DeltaV.GetClampedToSize(0.f, BuoyancySettings->MaxDeltaV);
			DeltaW = DeltaW.GetClampedToSize(0.f, BuoyancySettings->MaxDeltaW);

			// Compute new velocity
			const Chaos::FVec3 NewV = Submersion.Particle->GetV() + DeltaV;
			const Chaos::FVec3 NewW = Submersion.Particle->GetW() + DeltaW;

			// Apply the deltas
			Submersion.Particle->SetV(NewV);
			Submersion.Particle->SetW(NewW);

			// Wake up the body??
			if (BuoyancySettings->bKeepAwake)
			{
				Evolution.SetParticleObjectState(Submersion.Particle, Chaos::EObjectStateType::Dynamic);
			}
		}
	}
}

namespace
{
	bool IsParticleValid(Chaos::FGeometryParticleHandle* ParticleHandle, TSharedPtr<FProxyTimestampBase, ESPMode::ThreadSafe> SyncTimestamp)
	{
		if (bBuoyancyCallbackDataParticleValidation == false)
		{
			return true;
		}

		if (ParticleHandle == nullptr)
		{
			return false;
		}

		if (SyncTimestamp.IsValid() == false)
		{
			return false;
		}

		if (SyncTimestamp->bDeleted)
		{
			return false;
		}

		IPhysicsProxyBase* Proxy = ParticleHandle->PhysicsProxy();
		if (Proxy == nullptr)
		{
			return false;
		}

		if (!ensureMsgf(Proxy->GetSyncTimestamp() == SyncTimestamp, TEXT("particle's sync timestamp doesn't match the sync timestamp that was passed to the particle validity check!")))
		{
			return false;
		}

		return true;
	};
}

void FBuoyancySubsystemSimCallback::GenerateCallbackData()
{
	if (bBuoyancyCallbackDataEnabled == false)
	{
		return;
	}

	// Get refs to internal sparse arrays of submersion data
	TSparseArray<FBuoyancySubmersion>& PrevSubmersions = BuoyancyParticleData.PrevSubmersions;
	TSparseArray<FBuoyancySubmersionMetaData>& MetaDatas = BuoyancyParticleData.SubmersionMetaData;
	TSparseArray<FBuoyancySubmersionMetaData>& PrevMetaDatas = BuoyancyParticleData.PrevSubmersionMetaData;

	// Generate callback data if we're into that sort of thing
	const uint8 CallbackFlags = BuoyancySettings->SurfaceTouchCallbackFlags;
	if (CallbackFlags != 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_ProduceSurfaceTouches)

		// Get the async output struct to write to
		FBuoyancySubsystemSimCallbackOutput& Output = GetProducerOutputData_Internal();

		// Process every surface touch and queue up some of them to return
		// to game thread for callback dispatch
		Output.SurfaceTouches.Reserve(MetaDatas.Num() * FBuoyancySubmersionMetaData::MaxNumWaterContacts);
		for (auto Iter = MetaDatas.CreateIterator(); Iter; ++Iter)
		{
			const FBuoyancySubmersionMetaData& MetaData = *Iter;
			const int32 ObjectIndex = Iter.GetIndex();
			const FBuoyancySubmersion& Submersion = BuoyancyParticleData.Submersions[ObjectIndex];

			// Mark this as a new or continuing contact based on whether or
			// not we have a bit from the previous-submersions array.
			const bool bPrevSubmerged =
				PrevMetaDatas.IsValidIndex(ObjectIndex) &&
				PrevMetaDatas.IsAllocated(ObjectIndex);
			const EBuoyancyEventFlags TouchFlag
				= bPrevSubmerged
				? EBuoyancyEventFlags::Continue
				: EBuoyancyEventFlags::Begin;

			// Clear out the "prev" entry for this one's metadata so that
			// we can loop over the prev metadata for lost-contacts. Only
			// bother doing this work if we're tracking removals
			if (bPrevSubmerged && (CallbackFlags & EBuoyancyEventFlags::End) != 0)
			{
				PrevMetaDatas.RemoveAt(ObjectIndex);
			}

			// Only continue if we're tracking this touch type
			if ((TouchFlag & CallbackFlags) == 0)
			{
				continue;
			}

			if (!ensureMsgf(IsParticleValid(Submersion.Particle, Submersion.SyncTimestamp.Pin()), TEXT("Submersion data for buoyancy callback includes invalid submerged particle handle")))
			{
				continue;
			}

			// Build up output of new and continuing surface touches
			for (const FBuoyancySubmersionMetaData::FWaterContact& WaterContact : MetaData.WaterContacts)
			{
				if (!ensureMsgf(IsParticleValid(WaterContact.Water, WaterContact.SyncTimestamp.Pin()), TEXT("Submersion data for buoyancy callback includes invalid water body particle handle")))
				{
					continue;
				}

				Output.SurfaceTouches.Add({
					TouchFlag,
					Submersion.Particle->PhysicsProxy(),
					WaterContact.Water->PhysicsProxy(),
					WaterContact.Vol,
					WaterContact.CoM,
					WaterContact.Vel
				});
			}
		}

		// The remaining previous submersion metadata will correspond with lost contacts
		//
		// NOTE:
		// At the moment, lost contact callbacks will only occur when an entire object
		// loses contact, not just when one part of it loses contact.
		if ((CallbackFlags & EBuoyancyEventFlags::End) != 0)
		{
			for (auto Iter = PrevMetaDatas.CreateIterator(); Iter; ++Iter)
			{
				const FBuoyancySubmersionMetaData& MetaData = *Iter;
				const int32 ObjectIndex = Iter.GetIndex();
				const FBuoyancySubmersion& Submersion = PrevSubmersions[ObjectIndex];

				if (!IsParticleValid(Submersion.Particle, Submersion.SyncTimestamp.Pin()))
				{
					continue;
				}

				for (const FBuoyancySubmersionMetaData::FWaterContact& WaterContact : MetaData.WaterContacts)
				{
					if (!IsParticleValid(WaterContact.Water, WaterContact.SyncTimestamp.Pin()))
					{
						continue;
					}

					Output.SurfaceTouches.Add({
						EBuoyancyEventFlags::End,
						Submersion.Particle->PhysicsProxy(),
						WaterContact.Water->PhysicsProxy(),
						WaterContact.Vol,
						WaterContact.CoM,
						WaterContact.Vel
					});
				}
			}
		}
	}
}

#if WITH_BUOYANCY_MEMORY_TRACKING
void FBuoyancySubsystemSimCallback::GenerateAllocationData()
{
	SCOPE_CYCLE_COUNTER(STAT_BuoyancySubsystem_GenerateAllocationData)

	// NOTE: For now I'm guarding this inside WITH_BUOYANCY_MEMORY_TRACKING.
	// The current use case for the game thread GetAllocatedSize is a console
	// command which is editor-only. If this is to become useful
	// more generally, we can find a way to do it without relying on
	// continual calls to GetAllocatedSize().
	const uint32 NewAllocatedSize = GetAllocatedSize();
	if (AllocatedSize != NewAllocatedSize)
	{
		// If the number of allocated bytes has changed, update our cached num bytes
		AllocatedSize = NewAllocatedSize;

		// Get the async output struct to write to
		FBuoyancySubsystemSimCallbackOutput& Output = GetProducerOutputData_Internal();
		Output.AllocatedSize = AllocatedSize;
	}
}
#endif

