// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyWaterSplineKeyCacheGrid.h"
#include "CoreMinimal.h"
#include "BuoyancyStats.h"

float FSplineKeyCacheGrid::GetClosestSplineKey(const FBuoyancyWaterSplineData& WaterSplineData, const Chaos::FVec3& WorldPosition)
{
	LLM_SCOPE_BYTAG(BuoyancyParticleDataTag);

	// Convert the world position into a position local to the water spline
	const FVector LocalPosition = WaterSplineData.Transform.InverseTransformPosition(WorldPosition);

	// Get the grid corresponding with this particular spline
	TMap<Chaos::TVec3<int32>, float>& CacheMap = SplineKeyCache.FindOrAdd(&WaterSplineData);

	// Get the grid coordinates corresponding to the position
	const Chaos::TVec3<int32> CacheKey = GetCacheKey(LocalPosition);

	if (const float* ValuePtr = CacheMap.Find(CacheKey))
	{
		// If there's a cached value in this grid coordinate, return it
		return *ValuePtr;
	}
	else
	{
		// If we're going to exceed the cache limit by adding this item,
		// then remove an item from the map.
		if (CacheLimit > 0 && (uint32)CacheMap.Num() >= CacheLimit)
		{
			CacheMap.Reset();
		}

		// If there's no cached value in this grid coordinate, compute the
		// location of the center of the cell and get the spline key.
		const FVector LocalPos = GetLocalPos(CacheKey);
		float DistanceSq;
		const float SplineKey = WaterSplineData.Position.FindNearest(LocalPos, DistanceSq);
		CacheMap.Add(CacheKey) = SplineKey;
		return SplineKey;
	}
}

void FSplineKeyCacheGrid::Reset()
{
	SplineKeyCache.Reset();
}

void FSplineKeyCacheGrid::ForEachSplineKey(const TFunction<void(const FBuoyancyWaterSplineData&, const FVector&, float)>& Lambda)
{
	for (const TPair<const FBuoyancyWaterSplineData*, TMap<Chaos::TVec3<int32>, float>>& SplinePair : SplineKeyCache)
	{
		if (const FBuoyancyWaterSplineData* SplineData = SplinePair.Key)
		{
			for (const TPair<Chaos::TVec3<int32>, float> &KeyPair : SplinePair.Value)
			{
				Lambda(*SplineData, GetLocalPos(KeyPair.Key), KeyPair.Value);
			}
		}
	}
}

void FSplineKeyCacheGrid::SetGridSize(const float InGridSize)
{
	if (InGridSize < 1.f)
	{
		return;
	}

	GridSize = InGridSize;
	Reset();
}

void FSplineKeyCacheGrid::SetCacheLimit(const uint32 InCacheLimit)
{
	CacheLimit = InCacheLimit;
	Reset();
}

float FSplineKeyCacheGrid::GetGridSize() const
{
	return GridSize;
}

uint32 FSplineKeyCacheGrid::GetCacheLimit() const
{
	return CacheLimit;
}

SIZE_T FSplineKeyCacheGrid::GetAllocatedSize() const
{
	return SplineKeyCache.GetAllocatedSize();
}

Chaos::TVec3<int32> FSplineKeyCacheGrid::GetCacheKey(const FVector& LocalPos) const
{
	const float GridSizeInv = 1.f / GridSize;
	return Chaos::TVec3<int32>(
		FMath::FloorToInt32(LocalPos.X * GridSizeInv),
		FMath::FloorToInt32(LocalPos.Y * GridSizeInv),
		FMath::FloorToInt32(LocalPos.Z * GridSizeInv));
}

FVector FSplineKeyCacheGrid::GetLocalPos(const Chaos::TVec3<int32>& CacheKey) const
{
	return FVector(
		((float)CacheKey[0] + .5f) * GridSize,
		((float)CacheKey[1] + .5f) * GridSize,
		((float)CacheKey[2] + .5f) * GridSize);
}
