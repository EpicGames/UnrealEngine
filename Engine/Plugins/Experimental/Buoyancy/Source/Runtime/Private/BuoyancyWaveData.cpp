// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyWaveData.h"
#include "GerstnerWaveEvaluation.h"
#include "Misc/LargeWorldRenderPosition.h"

// ----------------------------------------------------------------------------------
// FBuoyancyGerstnerWaveData
// ----------------------------------------------------------------------------------

FBuoyancyGerstnerWaveData::FBuoyancyGerstnerWaveData(
	const TArray<FGerstnerWave>& InWaveParams,
	float InMaxWaveHeight,
	float InTargetWaveMaskDepth)
	: WaveParams(InWaveParams)
	, MaxWaveHeightValue(InMaxWaveHeight)
	, TargetWaveMaskDepth(InTargetWaveMaskDepth)
{
}

float FBuoyancyGerstnerWaveData::GetMaxWaveHeight() const
{
	return MaxWaveHeightValue;
}

float FBuoyancyGerstnerWaveData::GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InTime) const
{
	// Use the offset of the normalized tile as world position to match the shader behavior (see GerstnerWaveFunctions.ush).
	FVector WorldPosition(FLargeWorldRenderPosition(InPosition).GetOffset());
	return GerstnerWaveEvaluation::GetSimpleWaveHeightAtPosition(WaveParams, WorldPosition, InTime, /*bBlendLWCTiles=*/true);
}

float FBuoyancyGerstnerWaveData::GetWaveHeightAtPosition(const FVector& InPosition, float InTime, FVector& OutNormal) const
{
	// Use the offset of the normalized tile as world position to match the shader behavior (see GerstnerWaveFunctions.ush).
	FVector WorldPosition(FLargeWorldRenderPosition(InPosition).GetOffset());
	return GerstnerWaveEvaluation::GetWaveHeightAtPosition(WaveParams, WorldPosition, InTime, OutNormal, /*bBlendLWCTiles=*/true);
}

float FBuoyancyGerstnerWaveData::GetWaveAttenuationFactor(const FVector& Position, float InWaterDepth) const
{
	// Guard against TargetWaveMaskDepth=0 which would cause division by zero (NaN) in the
	// exponential. A zero mask depth means no attenuation is configured, so return full strength.
	if (TargetWaveMaskDepth <= UE_SMALL_NUMBER)
	{
		return 1.f;
	}
	return GerstnerWaveEvaluation::GetWaveAttenuationFactor(InWaterDepth, TargetWaveMaskDepth);
}
