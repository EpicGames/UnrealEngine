// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/Core.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/AABB.h"
#include "Chaos/Convex.h"
#include "WaterBodyComponent.h"


namespace Chaos
{
	class FPBDRigidsEvolutionGBF;
}

class FBuoyancyWaterSampler;
class FBuoyancyConstantSplineSampler;
class FBuoyancyShallowWaterSampler;
class FBuoyancyConstantSplineWithWavesSampler;

struct FBuoyancyParticleData;

namespace BuoyancyAlgorithms
{
	// Minimal struct containing essential data about a particular submersion
	struct FSubmersion
	{
		// Indicates the submerged particle
		Chaos::FPBDRigidParticleHandle* SubmergedParticle;

		// Total submerged volume
		float SubmergedVolume;

		// Effective submerged center of mass
		Chaos::FVec3 SubmergedCoM;
	};

	// #todo(dmp): adjust these
	struct FBuoyancyShapeTopologyLimits
	{
	public:
		static const int32 MaxVerticesPerShape = 50;		
		static const int32 MaxEdgesPerShape = 144;
		static const int32 MaxIntersectionPointsPerShape = MaxEdgesPerShape;
		static const int32 MaxVerticesPerFace = 10;
		static const int32 MaxSubmergedFaceVertices = MaxVerticesPerFace + 1;
	};
	
	// abstract class to sample water for a given particle
	class FBuoyancyShape
	{
	public:
		FBuoyancyShape() {};

		virtual ~FBuoyancyShape() = default;

		virtual void Initialize() = 0;	

		virtual int32 NumVertices() const = 0;
		virtual int32 NumEdges() const = 0;
		virtual int32 NumFaces() const = 0;
		virtual int NumFaceVertices(const int Index) const = 0;
		virtual int NumMaxIntersectionsPoints() const = 0;

		virtual Chaos::FVec3 GetVertex(int32 Index) const = 0;
		virtual void GetEdgeVertices(int EdgeIndex, int32& OutIndex0, int32& OutIndex1) const = 0;
		virtual int32 GetFaceVertex(const int FaceIndex, const int FaceVertexIndex) const = 0;
		virtual int32 GetFaceEdge(const int FaceIndex, const int FaceEdgeIndex) const = 0;
		
		virtual Chaos::FVec3 GetCenter() const = 0;
	};

	class FBuoyancyBoxShape : public FBuoyancyShape
	{
	public:
		FBuoyancyBoxShape(const Chaos::FAABB3& InLocalBox) : LocalBox(InLocalBox) {};
		
		void Initialize() {}

		int32 NumVertices() const override { return 8; }
		int32 NumEdges() const override { return 12; }
		int32 NumFaces() const override { return 6; }
		int NumFaceVertices(const int Index) const override { return 4; }
		int NumMaxIntersectionsPoints() const override { return 6; }

		Chaos::FVec3 GetVertex(int32 Index) const override { return LocalBox.GetVertex(Index); }

		void GetEdgeVertices(int EdgeIndex, int32& OutIndex0, int32& OutIndex1) const
		{
			const Chaos::FAABBEdge CurrEdge = LocalBox.GetEdge(EdgeIndex);
			OutIndex0 = CurrEdge.VertexIndex0;
			OutIndex1 = CurrEdge.VertexIndex1;
		}

		int32 GetFaceVertex(const int FaceIndex, const int FaceVertexIndex) const override { return LocalBox.GetFace(FaceIndex).VertexIndex[FaceVertexIndex]; }
		int32 GetFaceEdge(const int FaceIndex, const int FaceEdgeIndex) const override { return LocalBox.GetFace(FaceIndex).EdgeIndex[FaceEdgeIndex]; }		
		Chaos::FVec3 GetCenter() const override	{ return LocalBox.GetCenter(); }

	private:
		const Chaos::FAABB3& LocalBox;
	};

	class FBuoyancyConvexShape : public FBuoyancyShape
	{
	public:
		FBuoyancyConvexShape(const Chaos::FConvex *InConvex) : Convex(InConvex) {};

		virtual void Initialize() override;

		int32 NumVertices() const override { return Convex->NumVertices(); }
		int32 NumEdges() const override { return Convex->NumEdges(); }
		int32 NumFaces() const override { return Convex->GetFaces().Num(); }
		int NumFaceVertices(const int Index) const override { return Convex->NumPlaneVertices(Index); }
		
		// #todo(dmp): whats the best upper bound for max number of intersection points between a convex and plane?
		int NumMaxIntersectionsPoints() const override { return Convex->NumVertices(); }
		Chaos::FVec3 GetVertex(int32 Index) const override { return Convex->GetVertex(Index); }

		void GetEdgeVertices(int EdgeIndex, int32& OutIndex0, int32& OutIndex1) const
		{
			OutIndex0 = Convex->GetEdgeVertex(EdgeIndex, 0);
			OutIndex1 = Convex->GetEdgeVertex(EdgeIndex, 1);
		}

		int32 GetFaceVertex(const int FaceIndex, const int FaceVertexIndex) const override { return Convex->GetPlaneVertex(FaceIndex, FaceVertexIndex);	}
		int32 GetFaceEdge(const int FaceIndex, const int FaceEdgeIndex) const override
		{
			const int32 HalfEdgeIndex = Convex->GetPlaneHalfEdge(FaceIndex, FaceEdgeIndex);
			return HalfEdgeToEdge[HalfEdgeIndex];
		}
		
		Chaos::FVec3 GetCenter() const override	{ return Convex->GetCenterOfMass();	}

	private:
		const Chaos::FConvex* Convex;		
		
		TArray<int32, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxEdgesPerShape * 2>> HalfEdgeToEdge;
	};

	// Compute the effective volume of an entire particle based on its material
	// density and mass.
	Chaos::FRealSingle ComputeParticleVolume(const Chaos::FPBDRigidsEvolutionGBF& Evolution, const Chaos::FGeometryParticleHandle* Particle);

	// Compute the effective volume of a shape. This method must reflect the
	// maximum possible output value of the non-scaled ComputeSubmergedVolume.
	Chaos::FRealSingle ComputeShapeVolume(const Chaos::FGeometryParticleHandle* Particle, const bool UseBoundingBoxVolume = true);

	// Compute both bounding box and geometric shape volumes in a single traversal.
	void ComputeShapeVolumes(const Chaos::FGeometryParticleHandle* Particle, Chaos::FRealSingle& OutBBoxVol, Chaos::FRealSingle& OutGeomVol);

	//
	void ScaleSubmergedVolume(const Chaos::FPBDRigidsEvolutionGBF& Evolution, const Chaos::FGeometryParticleHandle* Particle, const bool UseBoundingBoxVolume, Chaos::FRealSingle& SubmergedVol, Chaos::FRealSingle& TotalVol);

	// Compute an approximate volume and center of mass of particle B submerged in particle A,
	// adjusting for the volume of the object based on the material density and mass of the object
	bool ComputeSubmergedVolume(FBuoyancyParticleData& ParticleData, const Chaos::FPBDRigidsEvolutionGBF& Evolution, const Chaos::FGeometryParticleHandle* ParticleA, const Chaos::FGeometryParticleHandle* ParticleB, int32 NumSubdivisions, float MinVolume, float& SubmergedVol, Chaos::FVec3& SubmergedCoM, float& TotalVol);

	// Compute an approximate volume and center of mass of particle B submerged in particle A
	bool ComputeSubmergedVolume(FBuoyancyParticleData& ParticleData, const Chaos::FGeometryParticleHandle* ParticleA, const Chaos::FGeometryParticleHandle* ParticleB, int32 NumSubdivisions, float MinVolume, float& SubmergedVol, Chaos::FVec3& SubmergedCoM);

	// Compute submerged volume given a single waterlevel
	bool ComputeSubmergedVolume(FBuoyancyParticleData& ParticleData, const Chaos::FPBDRigidsEvolutionGBF& Evolution, const Chaos::FGeometryParticleHandle* SubmergedParticle, const Chaos::FGeometryParticleHandle* WaterParticle, const FVector& WaterX, const FVector& WaterN, int32 NumSubdivisions, float MinVolume, float& SubmergedVol, Chaos::FVec3& SubmergedCoM, float& TotalVol);

	bool ComputeSubmergedVolume(FBuoyancyParticleData& ParticleData, const Chaos::FGeometryParticleHandle* SubmergedParticle, const Chaos::FGeometryParticleHandle* WaterParticle, const FVector& WaterX, const FVector& WaterN, int32 NumSubdivisions, float MinVolume, float& SubmergedVol, Chaos::FVec3& SubmergedCoM);

	// Given an OOBB and a water level, generate another OOBB which is 1. entirely contained
	// within the input OOBB and 2. entirely contains the portion of the OOBB which is submerged
	// below the water level.
	bool ComputeSubmergedBounds(const FVector& SurfacePointLocal, const FVector& SurfaceNormalLocal, const Chaos::FAABB3& RigidBox, Chaos::FAABB3& OutSubmergedBounds);

	// Given a bounds object, recursively subdivide it in eighths to a fixed maximum depth and
	// a fixed minimum smallest subdivision volume.
	bool SubdivideBounds(const Chaos::FAABB3& Bounds, int32 NumSubdivisions, float MinVolume, TArray<Chaos::FAABB3>& OutBounds);

	// Given a rigid particle and it's submerged CoM and Volume, compute delta velocities for
	// integrated buoyancy forces on an object
	bool ComputeBuoyantForce(const Chaos::FPBDRigidParticleHandle* RigidParticle, const float DeltaSeconds, const float WaterDensity, 
		const float WaterDrag, const Chaos::FVec3& GravityAccelVec, const Chaos::FVec3& SubmergedCoM, const float SubmergedVol, 
		const Chaos::FVec3& WaterVel, const Chaos::FVec3& WaterN, Chaos::FVec3& OutDeltaV, Chaos::FVec3& OutDeltaW);

	// given a particle, loop over the contained shapes and accumulate force/torque/submerged CoM values
	// Drag/lift and buoyancy forces are output separately to allow semi-implicit drag integration.
	template <typename SamplerType>
	void ComputeSubmergedVolumeAndForcesForParticle(FBuoyancyParticleData& ParticleData,
		const Chaos::FGeometryParticleHandle* SubmergedParticle, const Chaos::FGeometryParticleHandle* WaterParticle,
		TSharedPtr<SamplerType> WaterSampler,
		const Chaos::FPBDRigidsEvolution& Evolution, const float DeltaSeconds, const float WaterDensity, const float WaterDrag, const float WaterLift,
		float& OutTotalParticleVol, float& OutTotalSubmergedVol, Chaos::FVec3& OutTotalSubmergedCoM,
		Chaos::FVec3& OutTotalDragForce, Chaos::FVec3& OutTotalDragTorque,
		Chaos::FVec3& OutTotalBuoyancyForce, Chaos::FVec3& OutTotalBuoyancyTorque);

	// given a shape, compute the submerged volume and accumulate forces
	// this is done in a single function call because of the iterative nature of the algorithm
	template <typename ShapeType, typename SamplerType>
	void ComputeSubmergedVolumeAndForcesForShape(
		const Chaos::FGeometryParticleHandle* SubmergedParticle, const ShapeType& BoxShape,
		const Chaos::FPBDRigidsEvolution& Evolution, float DeltaSeconds, const float WaterDensity, const float WaterDrag, const float WaterLift,
		const float ParticleVol, const float ShapeVol,
		const Chaos::FRigidTransform3 &ShapeWorldTransform, TSharedPtr<SamplerType> WaterSampler,
		float& OutSubmergedVol, Chaos::FVec3& OutSubmergedCoM,
		Chaos::FVec3& OutForce, Chaos::FVec3& OutTorque,
		Chaos::FVec3& OutBuoyancyForce, Chaos::FVec3& OutBuoyancyTorque);

	// find intersection points between a plane and aabbox
	template <typename ShapeType>
	void FindAllIntersectionPoints(const Chaos::FVec3& WaterP, const Chaos::FVec3& WaterN, const ShapeType& BoxShape,
		const TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxVerticesPerShape>> &WorldVertexPosition,
		const TArray<bool, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxVerticesPerShape>> &VertexIsUnderwater,
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxEdgesPerShape>>& EdgeIntersectionPoints,
		int32& NumIntersections,
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxIntersectionPointsPerShape>>& OutOrderedIntersectionPoints, FVector& OutIntersectionCenter);

	// sort intersection points by angle
	void SortIntersectionPointsByAngle(const Chaos::FVec3& WaterN, const Chaos::FVec3& IntersectionCenter,
		const TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxEdgesPerShape>>& EdgeIntersectionPoints,
		int32 NumIntersections,
		TArray<FVector, TInlineAllocator<FBuoyancyShapeTopologyLimits::MaxIntersectionPointsPerShape>>& OutOrderedIntersectionPoints);

	bool EdgePlaneIntersection(const Chaos::FVec3& WaterP, const Chaos::FVec3& WaterN, const Chaos::FVec3& V0, const Chaos::FVec3& V1, Chaos::FVec3& IntersectionPoint);

	// compute area and volume of a tet from a triangle and center point on mesh
	FORCEINLINE void ComputeTriangleAreaAndVolume(const FVector &V0, const FVector &V1, const FVector &V2,
		const FVector &MeshCenterPoint, FVector& OutTriangleBaryCenter, FVector& OutNormal, float& OutArea, float& OutVolume)
	{
		// compute center of triangle
		OutTriangleBaryCenter = (V0 + V1 + V2) / 3.;

		// add up the volume of the tet created by this triangle and the center of the box
		const FVector A = V0 - MeshCenterPoint;
		const FVector B = V1 - MeshCenterPoint;
		const FVector C = V2 - MeshCenterPoint;
		const FVector N = A.Cross(B);

		OutVolume = FMath::Abs((N.Dot(C)) / 6.f);

		OutNormal = (V1 - V0).Cross(V2 - V0);
		float NormalLength = OutNormal.Length();

		// #todo(dmp): careful w/ divide by zero here?
		OutNormal /= NormalLength;

		// area of the triangle
		OutArea = .5 * NormalLength;
	}

	// compute the force the fluid exerts on a triangle
	FORCEINLINE void ComputeFluidForceForTriangle(const float WaterDrag, const float WaterLift,
		const float WaterDensity, const float DeltaSeconds,
		const Chaos::FVec3& ParticleV, const Chaos::FVec3& ParticleW, const FVector WorldCoM,
		const FVector &TriBaryCenter, const FVector &TriNormal, const float TriArea, const float TetVolume,
		const FVector &WaterVelocity, const FVector &WaterP, const FVector &WaterN,
		FVector& OutTotalWorldForce, FVector& OutTotalWorldTorque)
	{
		OutTotalWorldForce = FVector(0, 0, 0);
		OutTotalWorldTorque = FVector(0, 0, 0);

		const Chaos::FVec3 WorldForcePosition = TriBaryCenter;
		const Chaos::FVec3 WorldCOMToForcePos = WorldForcePosition - WorldCoM;

		// project velocity sample onto water plane to support waterfalls and flowing rivers more accurately
		const Chaos::FVec3 WaterVelocityOnPlane = WaterVelocity - WaterVelocity.Dot(WaterN) * WaterN;

		// Get world space particle linear velocity at current point
		// note we are including the linear velocity from torque so objects spin properly in flow
		const Chaos::FVec3 SubmergedParticleVelocity = ParticleV + Chaos::FVec3::CrossProduct(ParticleW, WorldCOMToForcePos);

		// compute force and torque to set linear velocity to fluid velocity
		const Chaos::FVec3 RelativeVelocity = WaterVelocityOnPlane - SubmergedParticleVelocity;
		const float RelativeVelocityMag = RelativeVelocity.Length();

		// test if this triangle is influenced by the water based on the normal since we have closed shapes only.  This drag algorithm
		// is derived for two sided planes
		const float FacingTest = RelativeVelocity.Dot(TriNormal);

		if (RelativeVelocityMag < SMALL_NUMBER || FacingTest < 0.f)
		{
			return;
		}

		// go with the flow combined drag and lift
		// https://www.yousufsoliman.com/projects/download/going-with-the-flow.pdf
		// const FVec3 ForceFromWaterVelocity = .5 * WaterDensity * RelativeVelocityMag * RelativeVelocity.Dot(TriNormal) * TriNormal * WaterDrag * TriArea;

		// https://www.cemyuksel.com/research/waveparticles/cem_yuksel_dissertation.pdf
		const float FluidA = TriArea * RelativeVelocity.Dot(TriNormal) / RelativeVelocityMag;
		const Chaos::FVec3 DragForce = .5 * WaterDensity * RelativeVelocityMag * RelativeVelocity * FluidA * WaterDrag;

		const Chaos::FVec3 LiftCrossProd = TriNormal.Cross(RelativeVelocity);
		const float LiftCrossProdLength = LiftCrossProd.Length();

		Chaos::FVec3 LiftForce = {0.0, 0.0, 0.0};
		if (LiftCrossProdLength > SMALL_NUMBER)
		{
			LiftForce = .5 * WaterDensity * RelativeVelocityMag * RelativeVelocity.Cross(LiftCrossProd / LiftCrossProdLength) * FluidA * WaterLift;
		}

		const Chaos::FVec3 ForceFromWaterVelocity = DragForce + LiftForce;

		// compute torque based on the linear force we apply
		const Chaos::FVec3 TorqueFromWaterVelocity = Chaos::FVec3::CrossProduct(WorldCOMToForcePos, ForceFromWaterVelocity);

		OutTotalWorldForce += ForceFromWaterVelocity;
		OutTotalWorldTorque += TorqueFromWaterVelocity;
	}

	// compute the force the buoyancy exerts on a shape
	void ComputeBuoyantForceForShape(const Chaos::FPBDRigidsEvolution& Evolution, const Chaos::FPBDRigidParticleHandle* RigidParticle, const float DeltaSeconds, const float WaterDensity,
		const float ParticleVol, const float ShapeVol,
		const Chaos::FVec3& SubmergedCoM, const float SubmergedVol, const Chaos::FVec3& WaterN, Chaos::FVec3& OutWorldBuoyantForce, Chaos::FVec3& OutWorldBuoyantTorque);
}
