// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "BuoyancyEventFlags.generated.h"

UENUM(BlueprintType, meta=(Bitflags, UseEnumValuesAsMaskValuesInEditor="true"))
enum EBuoyancyEventFlags : uint8
{
	None		= 0 UMETA(Hidden),
	Begin		= 1 << 0,
	Continue	= 1 << 1,
	End			= 1 << 2,
};
ENUM_CLASS_FLAGS(EBuoyancyEventFlags);
