// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Interface.h"
#include "BuoyancyEventInterface.generated.h"

#define UE_API BUOYANCY_API

UINTERFACE(MinimalAPI, Blueprintable)
class UBuoyancyEventInterface : public UInterface
{
	GENERATED_BODY()
};

class IBuoyancyEventInterface
{
	GENERATED_BODY()

public:

	/**
	* Called when part of a submerged simulated actor first comes in
	* contact with a water surface. This can be called for multiple parts
	* of a complex body.
	*/
	virtual void OnSurfaceTouchBegin_Native(
		UPrimitiveComponent* WaterComponent,
		UPrimitiveComponent* SubmergedComponent,
		float SubmergedVolume,
		const FVector& SubmergedCenterOfMass,
		const FVector& SubmergedVelocity) = 0;

	/**
	 * Called continually while objects maintain contact with a water surface.
	 * May be called multiple times for different parts of an object.
	 */
	virtual void OnSurfaceTouching_Native(
		UPrimitiveComponent* WaterComponent,
		UPrimitiveComponent* SubmergedComponent,
		float SubmergedVolume,
		const FVector& SubmergedCenterOfMass,
		const FVector& SubmergedVelocity) = 0;

	/**
	 * Called when a submerged body loses contact with all water surfaces. This
	 * can result from total submersion or from coming completely out of water.
	 */
	virtual void OnSurfaceTouchEnd_Native(
		UPrimitiveComponent* WaterComponent,
		UPrimitiveComponent* SubmergedComponent,
		float SubmergedVolume,
		const FVector& SubmergedCenterOfMass,
		const FVector& SubmergedVelocity) = 0;
	
	/**
	 * Called when part of a submerged simulated actor first comes in
	 * contact with a water surface. This can be called for multiple parts
	 * of a complex body.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = WaterBody)
	UE_API void OnSurfaceTouchBegin(
		class AWaterBody* WaterBodyActor,
		UPrimitiveComponent* WaterComponent,
		UPrimitiveComponent* SubmergedComponent,
		float SubmergedVolume,
		const FVector& SubmergedCenterOfMass,
		const FVector& SubmergedVelocity);

	/**
	 * Called continually while objects maintain contact with a water surface.
	 * May be called multiple times for different parts of an object.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = WaterBody)
	UE_API void OnSurfaceTouching(
		class AWaterBody* WaterBodyActor,
		UPrimitiveComponent* WaterComponent,
		UPrimitiveComponent* SubmergedComponent,
		float SubmergedVolume,
		const FVector& SubmergedCenterOfMass,
		const FVector& SubmergedVelocity);

	/**
	 * Called when a submerged body loses contact with all water surfaces. This
	 * can result from total submersion or from coming completely out of water.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = WaterBody)
	UE_API void OnSurfaceTouchEnd(
		class AWaterBody* WaterBodyActor,
		UPrimitiveComponent* WaterComponent,
		UPrimitiveComponent* SubmergedComponent,
		float SubmergedVolume,
		const FVector& SubmergedCenterOfMass,
		const FVector& SubmergedVelocity);
};

#undef UE_API
