// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/VelocityField.h"
#include "Chaos/SoftsExternalForces.h"
#include "BuoyancySubsystem.h"

using Chaos::Softs::FSolverVec3;
using Chaos::Softs::FSolverReal;

namespace Buoyancy
{
// Velocity field used solely for aerodynamics effects including water submersion
class FBuoyancyField : public Chaos::Softs::FVelocityAndPressureField
{
public:
	
	explicit FBuoyancyField(const Chaos::Softs::FCollectionPropertyConstFacade& PropertyCollection) : 
		Chaos::Softs::FVelocityAndPressureField(PropertyCollection),
		ClothDensityInWaterIndex(PropertyCollection),
		WaterDensityIndex(PropertyCollection),
		WaterTurbulenceRatioIndex(PropertyCollection)
	{
	}

	FBuoyancyField(
		const Chaos::Softs::FSolverParticlesRange& Particles,
		const Chaos::FTriangleMesh* TriangleMesh,
		const Chaos::Softs::FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<Chaos::FRealSingle>>& Weightmaps,
		Chaos::FSolverReal WorldScale, TSharedPtr<Chaos::Softs::FExternalForces> InExternalForces)
		: Chaos::Softs::FVelocityAndPressureField(Particles, TriangleMesh, PropertyCollection, Weightmaps, WorldScale),
		ClothDensityInWaterIndex(PropertyCollection),
		WaterDensityIndex(PropertyCollection),
		WaterTurbulenceRatioIndex(PropertyCollection),
		ExternalForces(InExternalForces)
		
	{		
	}

	// Construct an uninitialized field. Mesh, properties, and velocity will have to be set for this field to be valid.
	FBuoyancyField() : Chaos::Softs::FVelocityAndPressureField(),
		ClothDensityInWaterIndex(ForceInit),
		WaterDensityIndex(ForceInit),
		WaterTurbulenceRatioIndex(ForceInit)
	{
	}

	~FBuoyancyField() {}

	BUOYANCY_API void Apply(Chaos::Softs::FSolverParticlesRange& InParticles, const Chaos::FSolverReal Dt) const;
	
	BUOYANCY_API void SetPropertiesAndBuoyancy(
		const Chaos::Softs::FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<Chaos::FRealSingle>>& Weightmaps,
		FSolverReal WorldScale,
		bool bEnableAerodynamics,
		const FSolverVec3& SolverWind,
		const Chaos::FRotation3& LocalSpaceRotation = Chaos::FRotation3::Identity,
		const Chaos::Softs::FSolverReal LocalSpaceScale = (Chaos::Softs::FSolverReal)1.,
		const Chaos::FSolverVec3 LocalSpaceLocation = FSolverVec3::ZeroVector,
		const Chaos::FRotation3& ReferenceSpaceRotation = Chaos::FRotation3::Identity
		);

	BUOYANCY_API void PreSimulate_GameThread(float DeltaTime, const UWorld* World, const FBox BoundingBox);

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(ClothDensityInWater, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(WaterDensity, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(WaterTurbulenceRatio, float);

private:

	FSolverVec3 CalculateForce(FSolverVec3 N, const FSolverVec3& SurfaceVelocity, const FSolverReal SubmergedTriArea, const FSolverVec3& FluidVelocity, const FSolverReal CdI, const FSolverReal CdO, const FSolverReal ClI, const FSolverReal ClO, const FSolverReal Cp, const FSolverReal MaxVelocitySquared) const
	{		
		checkSlow(MaxVelocitySquared > (FSolverReal)0);
				
		// Calculate the direction and the relative velocity of the triangle to the flow
		
		FSolverVec3 V = FluidVelocity - SurfaceVelocity;

		// Clamp the velocity
		const FSolverReal RelVelocitySquared = V.SquaredLength();
		if (RelVelocitySquared > MaxVelocitySquared)
		{
			V *= FMath::Sqrt(MaxVelocitySquared / RelVelocitySquared);
		}

		return CalculateForceLowLevel(V, N, QuarterWaterRho, (FSolverReal) 2. * SubmergedTriArea, CdI, CdO, ClI, ClO, Cp, WaterTurbulenceRatio);
	}

	bool bUseWaterBodies = true;

	FBuoyancyCollisionData WaterBodyCollisions;
	FSolverReal LocalSpaceScale = 1.f;
	FSolverVec3 LocalSpaceLocation = {0.f,0.f,0.f};	
	FSolverReal ClothDensityInWater = 1000.f;
	FSolverReal WaterDensity = 1.f;
	FSolverReal WaterTurbulenceRatio = 1.f;

	FSolverReal WaterRho = 1.f;
	FSolverReal QuarterWaterRho = 1.f;
	
	FSolverReal ClothRho = 1.f;

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(ClothDensityInWater, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(WaterDensity, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(WaterTurbulenceRatio, float);

	TSharedPtr<Chaos::Softs::FExternalForces> ExternalForces;
};

}