// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "Containers/HashTable.h"
#include "CoreMinimal.h"
#include "Chaos/SimCallbackObject.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "PBDRigidsSolver.h"
#include "Engine/EngineBaseTypes.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "BuoyancyWaterSplineData.h"
#include "BuoyancyStats.h"

enum class EWaterSamplerType : uint8
{
	ConstantSpline,
	ShallowWater,
	ConstantSplineWithWaves,
	Invalid,
};

// abstract class to sample water for a given particle
class FBuoyancyWaterSampler
{
public:
	FBuoyancyWaterSampler() : SamplerType(EWaterSamplerType::Invalid), bUsePerShapeWaterPlane(true), 
		ClosestWaterPointToParticle(0,0,0), AverageVelocity(0,0,0)
	{
	};

	virtual ~FBuoyancyWaterSampler() = default;

	bool UsePerShapeWaterPlane() const { return bUsePerShapeWaterPlane;  }
	FVector GetClosestWaterPointToParticle() const { return ClosestWaterPointToParticle; }

	// average fluid velocity for the particle
	FVector GetAverageWaterVelocity() const { return AverageVelocity; }
	virtual void InitializeForSubmersion() {}
	virtual void SampleWaterAtPosition(const FVector& WorldPosition, FVector& OutVelocity, float& OutHeight, float& OutDepth) const = 0;
	virtual FVector SampleVelocityAtPosition(const FVector& WorldPosition) const = 0;
	virtual FVector SampleNormalAtPosition(const FVector& WorldPosition) const = 0;

	EWaterSamplerType GetSamplerType() const { return SamplerType; }
protected:
	EWaterSamplerType SamplerType;
	bool bUsePerShapeWaterPlane;
	FVector ClosestWaterPointToParticle;
	FVector AverageVelocity;
};

// sample water from a baked shallow water simulation
class FBuoyancyShallowWaterSampler : public FBuoyancyWaterSampler
{
public:
	FBuoyancyShallowWaterSampler(const TWeakObjectPtr<UShallowWaterSimulationDataBase> InWaterSampler, const FVector& InClosestWaterPointToParticle, const FVector &InAverageVelocity) :
		WaterSampler(InWaterSampler)
	{
		SamplerType = EWaterSamplerType::ShallowWater;
		bUsePerShapeWaterPlane = true;
		ClosestWaterPointToParticle = InClosestWaterPointToParticle;
		AverageVelocity = InAverageVelocity;
	}	

	virtual void SampleWaterAtPosition(const FVector& WorldPosition, FVector& OutVelocity, float& OutHeight, float& OutDepth) const override
	{
		if (WaterSampler.IsValid() && WaterSampler->HasValidData())
		{
			WaterSampler->SampleShallowWaterSimulationAtPosition(WorldPosition, OutVelocity, OutHeight, OutDepth);
		}
		else
		{
			OutVelocity = {0.f,0.f,0.f};
			OutHeight = 0.f;
			OutDepth = 0.f;
		}
	}

	virtual FVector SampleVelocityAtPosition(const FVector& WorldPosition) const override
	{
		FVector WaterVelocity = {0,0,0};

		if (WaterSampler.IsValid() && WaterSampler->HasValidData())
		{
			float TmpWaterVal;
			WaterSampler->SampleShallowWaterSimulationAtPosition(WorldPosition, WaterVelocity, TmpWaterVal, TmpWaterVal);
		}

		return WaterVelocity;
	}

	virtual FVector SampleNormalAtPosition(const FVector& WorldPosition) const override
	{
		if (WaterSampler.IsValid() && WaterSampler->HasValidData())
		{
			return WaterSampler->ComputeShallowWaterSimulationNormalAtPosition(WorldPosition);
		}
		else
		{
			return FVector(0,0,1);
		}
	}

private:
	const TWeakObjectPtr<UShallowWaterSimulationDataBase> WaterSampler;
};

// sampler for spline rivers, but we are only doing 1 spline sample per interaction to 
// match the perf of the old system.  Also, spline velocities don't vary much, so the net
// effect isn't too different
class FBuoyancyConstantSplineSampler : public FBuoyancyWaterSampler
{
public:
	FBuoyancyConstantSplineSampler(const FBuoyancyWaterSplineData& InWaterSpline, const FVector& InClosestPoint, 
		const FVector& InClosestPointDerivative, const float InClosestSplineKey, const FVector& InWaterN)
		 : WaterSpline(InWaterSpline), ClosestPointDerivative(InClosestPointDerivative), 
		   ClosestSplineKey(InClosestSplineKey), WaterN(InWaterN), WaterVel(FVector(0, 0, 0)), WaterVelSet(false)
	{
		SamplerType = EWaterSamplerType::ConstantSpline;
		bUsePerShapeWaterPlane = false;
		ClosestWaterPointToParticle = InClosestPoint;
		AverageVelocity = WaterVel;
	}

	virtual void InitializeForSubmersion() 
	{
		WaterVel = WaterSpline.Velocity.IsSet()
			? WaterSpline.Velocity->Eval(ClosestSplineKey) * ClosestPointDerivative.GetSafeNormal()
			: Chaos::FVec3::ZeroVector;

		AverageVelocity = WaterVel;
	}

	virtual void SampleWaterAtPosition(const FVector& WorldPosition, FVector& OutVelocity, float& OutHeight, float& OutDepth) const override
	{		
		OutVelocity = WaterVel;
		OutHeight = ClosestWaterPointToParticle.Z;

		// #todo(dmp): We technically don't use depth other than needing to know it is non zero
		// We should separate out the queries a bit more to avoid needing to set it here
		OutDepth = 1;
	}

	virtual FVector SampleVelocityAtPosition(const FVector& WorldPosition) const override
	{
		return WaterVel;
	}

	virtual FVector SampleNormalAtPosition(const FVector& WorldPosition) const override
	{
		return WaterN;
	}

public:	
	const FBuoyancyWaterSplineData &WaterSpline;
	
	FVector ClosestPointDerivative = FVector::ZeroVector;
	float ClosestSplineKey = 0;

	FVector WaterN = FVector::ZeroVector;
	FVector WaterVel = FVector::ZeroVector;
	bool WaterVelSet = false;
};

// Sampler for water bodies with Gerstner waves. Evaluates wave height and normal
// per-position using the wave data stored on the spline data struct.
class FBuoyancyConstantSplineWithWavesSampler : public FBuoyancyWaterSampler
{
public:
	FBuoyancyConstantSplineWithWavesSampler(
		const FBuoyancyWaterSplineData& InWaterSpline,
		const FVector& InParticlePosition,
		const FVector& InClosestPoint,
		const FVector& InClosestPointDerivative,
		const float InClosestSplineKey,
		const FVector& InWaterN,
		float InWaveReferenceTime,
		bool bInUseSimpleWaves = false)
		: WaterSpline(InWaterSpline)
		, SplineBaseHeight(InClosestPoint.Z)
		, ClosestPointDerivative(InClosestPointDerivative)
		, ClosestSplineKey(InClosestSplineKey)
		, BaseWaterN(InWaterN)
		, WaveReferenceTime(InWaveReferenceTime)
		, bUseSimpleWaves(bInUseSimpleWaves)
	{
		SamplerType = EWaterSamplerType::ConstantSplineWithWaves;
		bUsePerShapeWaterPlane = true;
		ClosestWaterPointToParticle = InClosestPoint;

		// Compute velocity from spline (same as ConstantSpline sampler)
		AverageVelocity = WaterSpline.Velocity.IsSet()
			? WaterSpline.Velocity->Eval(ClosestSplineKey) * ClosestPointDerivative.GetSafeNormal()
			: FVector::ZeroVector;

		// Estimate water depth at the particle's position.
		EstimatedDepth = InWaterSpline.EstimateWaterDepth(InParticlePosition, InClosestPoint, InClosestSplineKey);
		WaveAttenuationFactor = InWaterSpline.WaveData.IsValid()
			? InWaterSpline.WaveData->GetWaveAttenuationFactor(InParticlePosition, EstimatedDepth)
			: 1.f;
	}

	virtual void SampleWaterAtPosition(const FVector& WorldPosition, FVector& OutVelocity, float& OutHeight, float& OutDepth) const override
	{
		OutVelocity = AverageVelocity;

		// Base spline height + wave perturbation at query position, attenuated by depth
		float WaveHeight = 0.f;
		if (WaterSpline.WaveData.IsValid())
		{
			if (bUseSimpleWaves)
			{
				WaveHeight = WaterSpline.WaveData->GetSimpleWaveHeightAtPosition(WorldPosition, WaveReferenceTime);
			}
			else
			{
				FVector WaveNormal;
				WaveHeight = WaterSpline.WaveData->GetWaveHeightAtPosition(WorldPosition, WaveReferenceTime, WaveNormal);
			}
		}
		OutHeight = SplineBaseHeight + WaveHeight * WaveAttenuationFactor;

		OutDepth = EstimatedDepth;
	}

	virtual FVector SampleVelocityAtPosition(const FVector& WorldPosition) const override
	{
		return AverageVelocity;
	}

	virtual FVector SampleNormalAtPosition(const FVector& WorldPosition) const override
	{
		if (!bUseSimpleWaves && WaterSpline.WaveData.IsValid())
		{
			FVector WaveNormal;
			WaterSpline.WaveData->GetWaveHeightAtPosition(WorldPosition, WaveReferenceTime, WaveNormal);
			WaveNormal = FMath::Lerp(BaseWaterN, WaveNormal, WaveAttenuationFactor);
			return WaveNormal.IsZero() ? BaseWaterN : WaveNormal.GetSafeNormal();
		}
		return BaseWaterN;
	}

public:
	const FBuoyancyWaterSplineData& WaterSpline;
	float SplineBaseHeight = 0.f;
	FVector ClosestPointDerivative = FVector::ZeroVector;
	float ClosestSplineKey = 0.f;
	FVector BaseWaterN = FVector::ZeroVector;
	float WaveReferenceTime = 0.f;
	bool bUseSimpleWaves = false;
	float EstimatedDepth = MAX_FLT;
	float WaveAttenuationFactor = 1.f;
};

// Each particle will have a list of potential midphases to process,
// which must be sorted in descending Z order. This struct is used
// to store them
struct FBuoyancyInteraction
{
	Chaos::FPBDRigidParticleHandle* RigidParticle = nullptr;
	Chaos::FGeometryParticleHandle* WaterParticle = nullptr;
	TSharedPtr<FBuoyancyWaterSampler> WaterSampler;
};

static constexpr int32 MaxNumBuoyancyInteractions = 2;
using FBuoyancyInteractionArray = TArray<FBuoyancyInteraction, TInlineAllocator<MaxNumBuoyancyInteractions>>;

// A minimal struct of data tracking all the submersions in a frame.
struct FBuoyancySubmersion
{
	Chaos::FPBDRigidParticleHandle* Particle = nullptr;
	TWeakPtr<FProxyTimestampBase, ESPMode::ThreadSafe> SyncTimestamp = nullptr;
	
	float Vol = 0.f;
	FVector CoM = FVector::ZeroVector;

	FVector Vel = FVector::ZeroVector;
	FVector Norm = FVector::ZeroVector;

	// #todo(dmp): only used for accurate submersion calculations for now because we integrate those per body.  Ideally we'd
	// convert to only using this 
	bool HasIntegratedQuantities = false;
	FVector DeltaV = FVector::ZeroVector;
	FVector DeltaW = FVector::ZeroVector;
};

// Metadata for submersions, used for event callbacks
struct FBuoyancySubmersionMetaData
{
	struct FWaterContact
	{
		Chaos::FGeometryParticleHandle* Water = nullptr;
		TWeakPtr<FProxyTimestampBase, ESPMode::ThreadSafe> SyncTimestamp = nullptr;
		float Vol = 0.f;
		FVector CoM = FVector::ZeroVector;
		FVector Vel = FVector::ZeroVector;
	};

	// How many metadata's allowed per submerged particle
	static constexpr int32 MaxNumWaterContacts = 3;
	TArray<FWaterContact, TInlineAllocator<MaxNumWaterContacts>> WaterContacts;
};

struct FBuoyancyParticleData
{
private:

	// Access an element in a specific array, adding an uninitialized one if
	// it doesn't exist yet.
	template <typename TData>
	TData& GetData(const Chaos::FGeometryParticleHandle& ParticleHandle, TSparseArray<TData>& DataArray)
	{
		SCOPE_CYCLE_COUNTER(STAT_BuoyancyParticleData_GetData)

		const int32 DataIndex = AddOrGetIndex(ParticleHandle);

		if (DataArray.IsValidIndex(DataIndex) == false)
		{
			LLM_SCOPE_BYTAG(BuoyancyParticleDataTag);

			DataArray.Insert(DataIndex, TData());
		}

		return DataArray[DataIndex];
	}

public:

	FBuoyancyParticleData();
	~FBuoyancyParticleData();

	// Clear all data without freeing memory, for quick repopulation
	void Reset();

	// For memory debugging, log the number of bytes that this class has allocated
	SIZE_T GetAllocatedSize() const;

	// Internal index accessor
	int32 GetIndex(const Chaos::FGeometryParticleHandle& ParticleHandle);

	int32 AddOrGetIndex(const Chaos::FGeometryParticleHandle& ParticleHandle);

	bool RemoveIndex(const Chaos::FGeometryParticleHandle& ParticleHandle);

	// Bookkeeping arrays
	//
	// Map of indices from unique particle indices to internal array indices
	//TSparseArray<int32> IndexMap;
	FHashTable IndexMap;
	TSparseArray<int32> ReverseIndexMap;

	// Sparse array of arrays of buoyancy interactions - the outer array has one
	// entry per particle, the inner array has an entry per water body that it interacts
	// with. Each will be a very small array, sorted by Z.
	//
	// We use inline allocator to avoid more heap allocations, and to express the
	// assumption that a single particle is unlikely to exceed interactions with a
	// certain number of waterbodies at a time.
	TSparseArray<FBuoyancyInteractionArray> Interactions;

	// This sparse array of submersion events is indexed on particle unique indices.
	// All buoyant forces due to submersions are applied at once. It's stored as
	// a member variable and reset every frame, to avoid reallocation of similarly
	// sized data.
	TSparseArray<FBuoyancySubmersion> Submersions;
	TSparseArray<FBuoyancySubmersion> PrevSubmersions;

	// Another sparse array to be kept in sync with Submersions, which will contain
	// metadata useful for event callbacks
	TSparseArray<FBuoyancySubmersionMetaData> SubmersionMetaData;
	TSparseArray<FBuoyancySubmersionMetaData> PrevSubmersionMetaData;

	// This is a sparse array of bit arrays representing which shapes in an object
	// have already been accounted for when submerging an object. For example, if
	// a massive BVH object has two leaf node shapes submerged in different pools
	// of water and we've already detected that leaf A is submerged, we don't
	// need to test A again. This helps to avoid double counting submerged shapes.
	//
	// Just like Submersions, we have this as a member variable only to keep the
	// memory hot - the array is reset, repopulated and traversed, every frame,
	// so we want to minimize allocations.
	TSparseArray<TBitArray<>> SubmergedShapes;

	// Element accessors
	FBuoyancyInteractionArray& GetInteractions(const Chaos::FGeometryParticleHandle& ParticleHandle)
	{
		return GetData(ParticleHandle, Interactions);
	}

	FBuoyancySubmersion& GetSubmersion(const Chaos::FGeometryParticleHandle& ParticleHandle)
	{
		return GetData(ParticleHandle, Submersions);
	}

	FBuoyancySubmersion& GetPrevSubmersion(const Chaos::FGeometryParticleHandle& ParticleHandle)
	{
		return GetData(ParticleHandle, PrevSubmersions);
	}

	FBuoyancySubmersionMetaData& GetSubmersionMetaData(const Chaos::FGeometryParticleHandle& ParticleHandle)
	{
		return GetData(ParticleHandle, SubmersionMetaData);
	}

	FBuoyancySubmersionMetaData& GetPrevSubmersionMetaData(const Chaos::FGeometryParticleHandle& ParticleHandle)
	{
		return GetData(ParticleHandle, PrevSubmersionMetaData);
	}

	TBitArray<>& GetSubmergedShapes(const Chaos::FGeometryParticleHandle& ParticleHandle)
	{
		return GetData(ParticleHandle, SubmergedShapes);
	}

private:

	int32 GetIndex(const Chaos::FGeometryParticleHandle& ParticleHandle, int32& OutParticleIndex, int32& OutParticleKey);

	int32 GetIndex(const int32 ParticleIndex, const int32 ParticleKey);

	/** Shrink or grow internal memory - this operation may be slow when a resize is performed,
	    but should result in more optimal storage and faster data accesses on average. */
	void OptimizeMemory();
};
