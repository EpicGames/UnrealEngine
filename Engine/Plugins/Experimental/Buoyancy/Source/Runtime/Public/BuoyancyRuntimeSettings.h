// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Templates/SubclassOf.h"
#include "BuoyancyEventFlags.h"
#include "BuoyancyRuntimeSettings.generated.h"

#define UE_API BUOYANCY_API

enum ECollisionChannel : int;

UCLASS(MinimalAPI, Config = Engine, DefaultConfig, Meta = (DisplayName = "Buoyancy"))
class UBuoyancyRuntimeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UE_API virtual FName GetCategoryName() const;

	/** Whether or not the buoyancy subsystem should run.
	    The subsystem has a public interface for enabling/disabling, so
	    it's possible that a game overrides this at runtime */
	UPROPERTY(EditAnywhere, Config, Category = Buoyancy, Meta = (ClampMin = 0, ForceUnits = "g/cm3"))
	bool bBuoyancyEnabled = true;

	/** Prevent floating particles from falling asleep by explicitly setting
	    object state to dynamic when they are detected. */
	UPROPERTY(EditAnywhere, Config, Category = Buoyancy)
	bool bKeepFloatingObjectsAwake = false;

	/** Density of water to use in buoyancy calculations. The density of water is approximately 1g/cm3 */
	UPROPERTY(EditAnywhere, Config, Category = WaterProperties, Meta = (ClampMin = 0, ForceUnits = "g/cm3"))
	float WaterDensity = 1.f;

	/** Drag factor for submerged objects. This unitless number approximates the idea of viscosity,
	    but internally functions identically to the "ether drag" concept. */
	UPROPERTY(EditAnywhere, Config, Category = WaterProperties, Meta = (ClampMin = 0))
	float WaterDrag = 1.f;

	/** Lift coefficient for submerged surfaces. Controls how much angled surfaces
	    deflect objects perpendicular to the flow (rudder steering, planing).
	    0 = no lift (current behavior), 1 = standard lift. */
	UPROPERTY(EditAnywhere, Config, Category = WaterProperties, Meta = (ClampMin = 0))
	float WaterLift = 0.f;

	/** Collision channel to use for water ObjectTypes */
	UPROPERTY(EditAnywhere, Config, Category = WaterProperties)
	TEnumAsByte<ECollisionChannel> CollisionChannelForWaterObjects;

	/** Maximum number of times that a buoyancy bounds can be split into 8 */
	UPROPERTY(EditAnywhere, Config, Category = SubmergedVolumeCalculation, Meta = (ClampMin = 0))
	int32 MaxNumBoundsSubdivisions = 2;

	/** Minimum volume bounding box which can be produced by a subdivision operation */
	UPROPERTY(EditAnywhere, Config, Category = SubmergedVolumeCalculation)
	float MinBoundsSubdivisionVol = FMath::Pow(125.f, 3.f); // 1m^3

	/** Callback data for water surface contacts will be generated according to these flags */
	UPROPERTY(EditAnywhere, Config, Category = Callbacks, Meta = (Bitmask, BitmaskEnum = "/Script/Buoyancy.EBuoyancyEventFlags"))
	uint8 SurfaceTouchCallbackFlags = EBuoyancyEventFlags::Begin | EBuoyancyEventFlags::End;

	/** Minimum velocity of an object relative to a water required in order to trigger
	    a submersion callback. If this is zero, then the callback will be triggered
	    every frame for as long as any object is submerged. */
	UPROPERTY(EditAnywhere, Config, Category = Callbacks, Meta = (ClampMin = 0, ForceUnits = "cm/s", EditCondition = "bSubmersionCallbackEnabled"))
	float MinVelocityForSurfaceTouchCallback = 100.f;

	/** When enabled, cache computed water spline keys in a grid to avoid recalculation */
	UPROPERTY(EditAnywhere, Config, Category = Splines)
	bool bEnableSplineKeyCacheGrid = true;

	/** When using EnableSplineKeyCacheGrid, set grid size to this value. Larger means
		more caching/fewer spline evaluations, but less accurate water surface interactions. */
	UPROPERTY(EditAnywhere, Config, Category = Splines, Meta = (ClampMin = 1, ForceUnits = "cm"))
	float SplineKeyCacheGridSize = 300.f;

	/** Number of grid cells a single spline can cache at one time. After this limit has been
		exceeded, the cache will be reset. If the number of queries against a single body of
		water exceeds this number, then the caching system will likely fail to continue to
		provide performance benefits. */
	UPROPERTY(EditAnywhere, Config, Category = Splines, Meta = (ClampMin = 1))
	uint32 SplineKeyCacheLimit = 256;

	UE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateSettings, const UBuoyancyRuntimeSettings* /*Settings*/, EPropertyChangeType::Type /*ChangeType*/);
	static UE_API FOnUpdateSettings OnSettingsChange;
#endif
};

#undef UE_API
