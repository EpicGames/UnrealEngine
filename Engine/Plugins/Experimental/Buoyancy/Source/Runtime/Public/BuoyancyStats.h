// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("Buoyancy"), STATGROUP_Buoyancy, STATCAT_Advanced);

// Subsystem
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_OnPreSimulate"), STAT_BuoyancySubsystem_OnPreSimulate, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_OnMidPhaseModification"), STAT_BuoyancySubsystem_OnMidPhaseModification, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_VisitMidphases"), STAT_BuoyancySubsystem_VisitMidphases, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_TrackInteraction"), STAT_BuoyancySubsystem_TrackInteraction, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_DisableMidPhase"), STAT_BuoyancySubsystem_DisableMidPhase, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_AddInteraction"), STAT_BuoyancySubsystem_AddInteraction, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_SplineEvaluation"), STAT_BuoyancySubsystem_SplineEvaluation, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_SplineEvaluation_FindNearest"), STAT_BuoyancySubsystem_SplineEvaluation_FindNearest, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_SplineEvaluation_Eval"), STAT_BuoyancySubsystem_SplineEvaluation_Eval, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_BuildSubmersions"), STAT_BuoyancySubsystem_BuildSubmersions, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_BuildSubmersionCallbackData"), STAT_BuoyancySubsystem_BuildSubmersionCallbackData, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_ApplyBuoyantForces"), STAT_BuoyancySubsystem_ApplyBuoyantForces, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_ProduceSurfaceTouches"), STAT_BuoyancySubsystem_ProduceSurfaceTouches, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_GenerateAllocationData"), STAT_BuoyancySubsystem_GenerateAllocationData, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_Tick"), STAT_BuoyancySubsystem_Tick, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_UpdateBuoyancySettings"), STAT_BuoyancySubsystem_UpdateBuoyancySettings, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_UpdateWaterBodiesList"), STAT_BuoyancySubsystem_UpdateWaterBodiesList, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_DispatchCallbacks"), STAT_BuoyancySubsystem_DispatchCallbacks, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_ShallowWaterIntegrateForces"), STAT_BuoyancySubsystem_ShallowWaterIntegrateForces, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_ProcessInteractions"), STAT_BuoyancySubsystem_ProcessInteractions, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Subsystem_ProcessInteraction"), STAT_BuoyancySubsystem_ProcessInteraction, STATGROUP_Buoyancy);

// Algorithms
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolume"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolume, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedBounds"), STAT_BuoyancyAlgorithms_ComputeSubmergedBounds, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ScaleSubmergedVolume"), STAT_BuoyancyAlgorithms_ScaleSubmergedVolume, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_SubdivideBounds"), STAT_BuoyancyAlgorithms_SubdivideBounds, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeBuoyantForces"), STAT_BuoyancyAlgorithms_ComputeBuoyantForces, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_SubmergeShapeInternal"), STAT_BuoyancyAlgorithms_SubmergeShapeInternal, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolumeAndForcesForParticle"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForParticle, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolumeAndForcesForShape"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolumeAndForcesForShape_ComputeWorld"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_ComputeWorld, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolumeAndForcesForShape_FaceIteration"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_FaceIteration, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolumeAndForcesForShape_FindFaceIntersections"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_FindFaceIntersections, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolumeAndForcesForShape_ComputeFaceForces"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_ComputeFaceForces, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolumeAndForcesForShape_SampleVelocity"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_SampleVelocity, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_FindAllIntersectionPoints"), STAT_BuoyancyAlgorithms_FindAllIntersectionPoints, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_SortIntersectionPointsByAngle"), STAT_BuoyancyAlgorithms_SortIntersectionPointsByAngle, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeTriangleAreaAndVolume"), STAT_BuoyancyAlgorithms_ComputeTriangleAreaAndVolume, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeFluidForceForTriangle"), STAT_BuoyancyAlgorithms_ComputeFluidForceForTriangle, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeBuoyantForceForShape"), STAT_BuoyancyAlgorithms_ComputeBuoyantForceForShape, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_FBuoyancyConvexShape_Initialize"), STAT_BuoyancyAlgorithms_FBuoyancyConvexShape_Initialize, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_Algorithms_ComputeSubmergedVolumeAndForcesForShape_FindFaceVerticesBelowWater"), STAT_BuoyancyAlgorithms_ComputeSubmergedVolumeAndForcesForShape_FindFaceVerticesBelowWater, STATGROUP_Buoyancy);

// Particle Data
DECLARE_CYCLE_STAT(TEXT("Buoyancy_ParticleData_GetData"), STAT_BuoyancyParticleData_GetData, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_ParticleData_Reset"), STAT_BuoyancyParticleData_Reset, STATGROUP_Buoyancy);
DECLARE_CYCLE_STAT(TEXT("Buoyancy_ParticleData_OptimizeMemory"), STAT_BuoyancyParticleData_OptimizeMemory, STATGROUP_Buoyancy);

// Memory tracking tags
LLM_DECLARE_TAG(BuoyancyParticleDataTag);
