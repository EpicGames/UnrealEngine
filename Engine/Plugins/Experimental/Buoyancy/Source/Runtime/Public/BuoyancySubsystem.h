// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "Chaos/SimCallbackObject.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "PBDRigidsSolver.h"
#include "Engine/EngineBaseTypes.h"
#include "WaterBodyComponent.h"
#include "BuoyancyWaterSplineData.h"
#include "BuoyancyWaterSplineKeyCacheGrid.h"
#include "BuoyancyEventFlags.h"
#include "ChaosUserDataPT.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "BuoyancyParticleData.h"
#include "BuoyancyMacros.h"
#include "BuoyancySubsystem.generated.h"

#define UE_API BUOYANCY_API

DECLARE_LOG_CATEGORY_EXTERN(LogBuoyancySubsystem, Log, All);

//
// Callback object for keeping water splines up to date on the physics thread
//
// NOTE: We use shared ptr here because a single water spline might have many
// particles associated with it, but we'd like to only store a single copy
// of the spline.
//

class FBuoyancyWaterSplineDataManager : public Chaos::TUserDataManagerPT< TSharedPtr<FBuoyancyWaterSplineData> > { };

namespace Chaos
{
	class FMidPhaseModifierAccessor;
	class FMidPhaseModifier;
}


//
// Buoyancy Settings
//

struct FBuoyancySettings
{
	// Force buoyant particles which are in water to stay awake
	bool bKeepAwake = false;

	// Density of water is about 1g/cm^3
	// Source: https://en.wikipedia.org/wiki/Properties_of_water
	float WaterDensity = 0.0001f; // kg/cm^3

	float MaxDeltaV = 500.f; // cm/s

	float MaxDeltaW = 5.f; // rad/s

	float WaterDrag = 1.f; // unitless

	float WaterLift = 0.f; // unitless

	int32 MaxNumBoundsSubdivisions = 2;

	float MinBoundsSubdivisionVol = FMath::Pow(125.f, 3.f); // 1m^3

	ECollisionChannel WaterCollisionChannel = ECollisionChannel::ECC_MAX;

	uint8 SurfaceTouchCallbackFlags = EBuoyancyEventFlags::None;

	float MinVelocityForSurfaceTouchCallback = 10.f;

	bool bSplineKeyCacheGrid = true;

	float SplineKeyCacheGridSize = 300.f;

	uint32 SplineKeyCacheLimit = 256;
};

class FBuoyancyCollisionData
{
	public:
		FBuoyancyCollisionData(UBuoyancySubsystem *InTheBuoyancySubsystem, 
			TArray<const TSharedPtr<FBuoyancyWaterSplineData>> InWaterBodyCollisionData, FBox InWorldBounds,
			bool InHasCollision,
			bool InHasPlane,
			FVector InWaterPlaneLocation,
			FVector InWaterPlaneNormal,
			FVector InWaterPlaneVelocity)
		: TheBuoyancySubsystem(InTheBuoyancySubsystem)
		, WaterBodyCollisionData(InWaterBodyCollisionData)
		, WorldBounds(InWorldBounds)
		, HasCollision(InHasCollision)
		, HasPlane(InHasPlane)
		, WaterPlaneLocation(InWaterPlaneLocation)
		, WaterPlaneNormal(InWaterPlaneNormal)
		, WaterPlaneVelocity(InWaterPlaneVelocity)
		{}
		
		FBuoyancyCollisionData() 
		: TheBuoyancySubsystem(nullptr)
		, WaterBodyCollisionData(TArray<const TSharedPtr<FBuoyancyWaterSplineData>>()) 
		, WorldBounds()
		, HasCollision(false)
		, HasPlane(false)
		, WaterPlaneLocation(0,0,0)
		, WaterPlaneNormal(0,0,1)
		, WaterPlaneVelocity(0,0,0)
		{}

		UBuoyancySubsystem *TheBuoyancySubsystem;
		TArray<const TSharedPtr<FBuoyancyWaterSplineData>> WaterBodyCollisionData;
		FBox WorldBounds;
		bool HasCollision;
		bool HasPlane;
		FVector WaterPlaneLocation;
		FVector WaterPlaneNormal;
		FVector WaterPlaneVelocity;
};

//
// Buoyancy Subsystem
//

UCLASS(MinimalAPI)
class UBuoyancySubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

		UBuoyancySubsystem()
		: UTickableWorldSubsystem()
		, bWaterObjectsChanged(false)
		, bBuoyancySettingsChanged(false)
		, BuoyancySettings(FBuoyancySettings())
		, SplineData(nullptr)
		, SimCallback(nullptr)
		, NetMode(ENetMode::NM_MAX)
	{ }

public:

	// Return true if enable/disable was successful, or if
	// we were already in the target state.
	UE_API bool SetEnabled(const bool bEnabled);

	// Return true if subsystem is enabled and running
	UFUNCTION()
	UE_API bool IsEnabled() const;

	UE_API bool SetEnabledWithUpdatedNetModeCallback(const bool bEnabled);

#if WITH_BUOYANCY_MEMORY_TRACKING
	UE_API SIZE_T GetAllocatedSize() const;
#endif

	// Given a bounding box, perform an overlap test and filter for water bodies, returning the data needed for buoyancy.  
	// There are threading issues here since we are storing pointers to FBuoyancyWaterSplineDataManager
	// physics thread data on the gt to use for tests later on (with FindOverlappingWaterBodies).  
	// Ideally the Cloth solver would be able to make use of TUserDataManagerPT even if it doesn't run on the pt	
	UE_API bool FindOverlappingWaterBodies(
		const FBox BoundingBox, 
		bool ComputePlane, 
		FVector &ClosestWaterPlaneLocation,
		FVector &ClosestWaterPlaneNormal,
		FVector &ClosestWaterPlaneVelocity,
		TArray<const TSharedPtr<FBuoyancyWaterSplineData>> &CollisionWaterBodySplineData);

	// Query a water body given a FBuoyancyWaterSplineData object representing a water body
	// #todo(dmp): This is designed to only work on the physics thread, needs to be more robust if used elsewhere
	// IE: Ideally the Cloth solver would be able to make use of TUserDataManagerPT even if it doesn't run on the pt
	// and it'd need to support the spline cache
	UE_API bool QueryWaterBody(const FVector& InputPosition,  const TSharedPtr<FBuoyancyWaterSplineData> WaterData,
		FVector& WaterVel, FVector& WaterPlaneN, FVector& WaterPlanePos);

protected:

	// UTickableWorldSubsystem begin interface
	UE_API virtual void PostInitialize() override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;
	// UTickableWorldSubsystem end interface

	// FWaterBodyManager delegate begin callbacks
	UE_API void OnWaterBodyAdded(UWaterBodyComponent* WaterBodyComponent);
	UE_API void OnWaterBodyRemoved(UWaterBodyComponent* WaterBodyComponent);
	// FWaterBodyManager delegate end callbacks

private:

	void CreateSimCallback();
	void DestroySimCallback();

	// Self explanatory function name :D
	void UpdateAllAsyncInputs();

	// Get netmode from world and send it to PT
	void UpdateNetMode();

	// Update PT spline data structs for each waterbody in the map
	void UpdateSplineData();

	// Put updated settings struct onto async input to be sent to sim callback
	void UpdateBuoyancySettings();

	// Send wave reference time to physics thread each frame
	void UpdateWaveReferenceTime();

	// Process async outputs which hold data for triggering callbacks
	void ProcessSurfaceTouchCallbacks();

	Chaos::FPhysicsSolver* GetSolver() const;

	// When water plugin settings change, this callback will apply changes
	void ApplyRuntimeSettings(const class UBuoyancyRuntimeSettings* InSettings, EPropertyChangeType::Type ChangeType);

	bool bWaterObjectsChanged;

	bool bBuoyancySettingsChanged;

	// Cached pointer to r.Water.OceanFallbackDepth CVar (looked up once in PostInitialize)
	IConsoleVariable* OceanFallbackDepthCVar = nullptr;

	FBuoyancySettings BuoyancySettings;

	FBuoyancyWaterSplineDataManager* SplineData;

	class FBuoyancySubsystemSimCallback* SimCallback;

	ENetMode NetMode;

#if WITH_BUOYANCY_MEMORY_TRACKING
	uint32 SimCallback_AllocatedSize;
#endif
};


//
// Buoyancy Sim Callback
//

struct FBuoyancySubsystemSimCallbackInput : public Chaos::FSimCallbackInput
{
	// If this ptr is set, then we have a new spline data manager...
	// That should only probably happen one time
	TOptional<FBuoyancyWaterSplineDataManager*> SplineData;

	// Here we use a unique ptr so that it is possible to provide an async
	// input _without_ buoyancy settings (which may be eventually desirable
	// when we eventually are passing lists of water bodies or water wave
	// data).
	mutable TUniquePtr<FBuoyancySettings> BuoyancySettings;

	// Set when net mode changes - should be one time on initialization.
	TOptional<ENetMode> NetMode;

	// Wave reference time, sent every frame when waves are active
	TOptional<float> WaveReferenceTime;

	void Reset();
};

struct FBuoyancySubsystemSimCallbackOutput : public Chaos::FSimCallbackOutput
{
	struct FSurfaceTouch
	{
		uint8 Flag;
		IPhysicsProxyBase* RigidProxy;
		IPhysicsProxyBase* WaterProxy;
		float Vol;
		FVector CoM;
		FVector Vel;
	};

	TArray<FSurfaceTouch> SurfaceTouches;

#if WITH_BUOYANCY_MEMORY_TRACKING
	// Optional update to allocated num bytes
	TOptional<uint32> AllocatedSize;
#endif

	void Reset();
};

// NOTE: The Presimulate option is only needed for proper registry with the solver.
//       We don't actually need (or want!) a presimulate tick.
class FBuoyancySubsystemSimCallback : public Chaos::TSimCallbackObject<
	FBuoyancySubsystemSimCallbackInput,
	FBuoyancySubsystemSimCallbackOutput,
	Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::MidPhaseModification>
{
public:

	SIZE_T GetAllocatedSize() const;
	FSplineKeyCacheGrid &GetSplineKeyCache() { return SplineKeyCache; }

	bool QuerySpline(const FVector &QueryPos, const FBuoyancyWaterSplineData& WaterSpline, float &ClosestSplineKey,
		FVector &ClosestPoint, FVector &ClosestPointDerivative, FVector &WaterN);
private:

	virtual void OnPreSimulate_Internal() override;
	virtual void OnMidPhaseModification_Internal(Chaos::FMidPhaseModifierAccessor& Modifier) override;

	void TrackInteractions(Chaos::FPBDRigidsSolver& PBDSolver, Chaos::FPBDRigidsEvolution& Evolution, Chaos::FMidPhaseModifierAccessor& MidPhaseAccessor);
	
	void TrackInteraction(Chaos::FPBDRigidsEvolution& Evolution, Chaos::FGeometryParticleHandle* WaterParticle, Chaos::FPBDRigidParticleHandle* RigidParticle, const FBuoyancyWaterSplineData& WaterSpline, Chaos::FMidPhaseModifier& MidPhase);
	void ProcessInteractions(Chaos::FPBDRigidsEvolution& Evolution);
	void ProcessAccurateInteraction(Chaos::FPBDRigidsEvolution& Evolution, FBuoyancyInteraction& Interaction, TSharedPtr<FBuoyancyWaterSampler> WaterSampler);
	void ProcessInteraction(Chaos::FPBDRigidsEvolution& Evolution, FBuoyancyInteraction& Interaction);
	void AddSurfaceTouchCallback(FBuoyancyInteraction& Interaction, FBuoyancySubmersion& Submersion, float TotalVol, float SubmergedVol, FVector SubmergedCoM);
	void ApplyBuoyantForces(Chaos::FPBDRigidsEvolution& Evolution);
	void GenerateCallbackData();
	
#if WITH_BUOYANCY_MEMORY_TRACKING
	void GenerateAllocationData();
	SIZE_T AllocatedSize = 0;
#endif

	// Reference to UserDataPT sim callback which manages synchronization of
	// water spline data
	FBuoyancyWaterSplineDataManager* SplineData = nullptr;

	// Initially we won't have any settings - they have to get passed down
	// via async input. I used TUniquePtr to control access to the same
	// memory that was allocated by GT to minimize copies.
	TUniquePtr<FBuoyancySettings> BuoyancySettings;

	FBuoyancyParticleData BuoyancyParticleData;

	// Used to track the net mode of the world that owns the phys scene that this
	// sim tick is taking place in.
	ENetMode NetMode = ENetMode::NM_MAX;

	// Wave reference time for evaluating wave heights on the physics thread
	float WaveReferenceTime = 0.0f;

	// Local cache of spline keys to reduce spline evaluations.
	FSplineKeyCacheGrid SplineKeyCache;
};

#undef UE_API
