// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyComponent.h"
#include "BuoyancyWaveData.h"

struct FBuoyancyWaterSplineData
{
	FBuoyancyWaterSplineData() {}
	FBuoyancyWaterSplineData(
		const Chaos::FRigidTransform3& InTransform,
		const FInterpCurveVector& InPosition,
		const EWaterBodyType InBodyType,
		const TOptional<FInterpCurveFloat>& InWidth,
		const TOptional<FInterpCurveFloat>& InVelocity,
		const TWeakObjectPtr<UShallowWaterSimulationDataBase> InShallowWaterSimData,
		TSharedPtr<FBuoyancyWaveData> InWaveData = nullptr,
		const TOptional<FInterpCurveFloat>& InDepth = TOptional<FInterpCurveFloat>(),
		float InChannelDepth = 0.f,
		float InCurveRampWidth = 512.f,
		float InChannelEdgeOffset = 0.f,
		float InOceanFallbackDepth = 0.f
	)
		: Transform(InTransform)
		, Position(InPosition)
		, BodyType(InBodyType)
		, Width(InWidth)
		, Velocity(InVelocity)
		, ShallowWaterSimData(InShallowWaterSimData)
		, WaveData(InWaveData)
		, Depth(InDepth)
		, ChannelDepth(InChannelDepth)
		, CurveRampWidth(InCurveRampWidth)
		, ChannelEdgeOffset(InChannelEdgeOffset)
		, OceanFallbackDepth(InOceanFallbackDepth)
	{ }


	bool ShouldSampleFromShallowWaterSimulation() const
	{
		return ShallowWaterSimData.IsValid() && ShallowWaterSimData->HasValidData();
	}

	// Parameters that all water bodies have
	Chaos::FRigidTransform3 Transform;
	FInterpCurveVector Position;
	EWaterBodyType BodyType;

	// Parameters that only _some_ water bodies have
	TOptional<FInterpCurveFloat> Width;
	TOptional<FInterpCurveFloat> Velocity;

	// Only some water bodies can have a baked shallow water sim representation, which
	// overrides the splines for water height/depth/velocity/normal
	const TWeakObjectPtr<UShallowWaterSimulationDataBase> ShallowWaterSimData;

	// Wave data (null if water body has no waves)
	TSharedPtr<FBuoyancyWaveData> WaveData;

	// For rivers: actual depth curve from spline metadata
	TOptional<FInterpCurveFloat> Depth;

	// For ocean/lake: estimate depth from distance to shoreline spline
	float ChannelDepth = 0.f;
	float CurveRampWidth = 512.f;
	float ChannelEdgeOffset = 0.f;

	// For ocean: fallback depth when beyond terrain carving zone
	float OceanFallbackDepth = 0.f;

	bool HasWaves() const { return WaveData.IsValid(); }

	// Estimate water depth at a world position given closest spline point and key.
	// Rivers use the actual depth spline curve; ocean/lake estimate from XY distance to shoreline.
	float EstimateWaterDepth(const FVector& WorldPosition, const FVector& ClosestSplinePoint, float SplineKey) const
	{
		if (BodyType == EWaterBodyType::River && Depth.IsSet())
		{
			// Rivers: use actual depth curve from spline metadata
			return FMath::Max(Depth->Eval(SplineKey, 0.f), 0.f);
		}

		// Ocean/Lake: estimate from XY distance to shoreline spline.
		// Terrain carving starts at ChannelEdgeOffset from the spline and
		// ramps to ChannelDepth over CurveRampWidth.
		const float DistToSpline = FVector::Dist2D(WorldPosition, ClosestSplinePoint);
		const float EffectiveDist = FMath::Max(DistToSpline - ChannelEdgeOffset, 0.f);
		const float SafeRampWidth = FMath::Max(CurveRampWidth, 1.f);

		if (BodyType == EWaterBodyType::Ocean)
		{
			// Ocean: depth ramps from 0 at the carving edge to ChannelDepth,
			// then continues toward OceanFallbackDepth for deep water.
			const float FallbackDepth = FMath::Max(OceanFallbackDepth, ChannelDepth);
			if (FallbackDepth <= 0.f)
			{
				return MAX_FLT;
			}
			if (ChannelDepth > 0.f)
			{
				// Within carving ramp: linear ramp to ChannelDepth
				if (EffectiveDist <= SafeRampWidth)
				{
					return ChannelDepth * (EffectiveDist / SafeRampWidth);
				}
				// Beyond carving: transition from ChannelDepth to FallbackDepth
				const float T = FMath::Clamp((EffectiveDist - SafeRampWidth) / SafeRampWidth, 0.f, 1.f);
				return FMath::Lerp(ChannelDepth, FallbackDepth, T);
			}
			// No carving configured: ramp to fallback depth
			return FallbackDepth * FMath::Clamp(EffectiveDist / SafeRampWidth, 0.f, 1.f);
		}

		// Lake/other: depth caps at ChannelDepth (lake bottom = carved terrain)
		if (ChannelDepth <= 0.f)
		{
			return MAX_FLT;
		}
		return ChannelDepth * FMath::Clamp(EffectiveDist / SafeRampWidth, 0.f, 1.f);
	}
};