// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BuoyancyWaterSplineData.h"

class FSplineKeyCacheGrid
{
public:

	float GetClosestSplineKey(const FBuoyancyWaterSplineData& WaterSplineData, const Chaos::FVec3& WorldPosition);

	void Reset();

	void ForEachSplineKey(const TFunction<void(const FBuoyancyWaterSplineData&, const FVector&, float)>& Lambda);

	void SetGridSize(const float InGridSize);
	void SetCacheLimit(const uint32 InCacheLimit);

	float GetGridSize() const;
	uint32 GetCacheLimit() const;

	SIZE_T GetAllocatedSize() const;

private:

	Chaos::TVec3<int32> GetCacheKey(const FVector& LocalPos) const;

	FVector GetLocalPos(const Chaos::TVec3<int32>& CacheKey) const;

	float GridSize = 300.f;

	uint32 CacheLimit = 256;

	using FSplineKeyCache = TMap<const FBuoyancyWaterSplineData*, TMap<Chaos::TVec3<int32>, float>>;
	FSplineKeyCache SplineKeyCache;
};
