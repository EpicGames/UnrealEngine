// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GerstnerWaterWaves.h"

// Abstract base for wave data evaluable on the physics thread (no UObject access).
// Copied from game-thread UWaterWaves data so that the physics thread can evaluate
// wave heights without touching UObjects.
struct FBuoyancyWaveData
{
	virtual ~FBuoyancyWaveData() = default;

	virtual float GetMaxWaveHeight() const = 0;

	// Fast version: height only, no normal perturbation
	virtual float GetSimpleWaveHeightAtPosition(
		const FVector& InPosition, float InTime) const = 0;

	// Full version: height + perturbed normal
	virtual float GetWaveHeightAtPosition(
		const FVector& InPosition, float InTime, FVector& OutNormal) const = 0;

	// Attenuation factor (0-1) based on water depth (e.g. shore dampening)
	virtual float GetWaveAttenuationFactor(
		const FVector& InPosition, float InWaterDepth) const = 0;
};

// Gerstner wave implementation - self-contained copy of wave params for physics thread use.
// Math mirrors UGerstnerWaterWaves::GetWaveHeightAtPosition / GetSimpleWaveHeightAtPosition.
struct FBuoyancyGerstnerWaveData : public FBuoyancyWaveData
{
	TArray<FGerstnerWave> WaveParams;
	float MaxWaveHeightValue = 0.0f;
	float TargetWaveMaskDepth = 0.0f;

	FBuoyancyGerstnerWaveData(
		const TArray<FGerstnerWave>& InWaveParams,
		float InMaxWaveHeight,
		float InTargetWaveMaskDepth);

	virtual float GetMaxWaveHeight() const override;
	virtual float GetSimpleWaveHeightAtPosition(
		const FVector& InPosition, float InTime) const override;
	virtual float GetWaveHeightAtPosition(
		const FVector& InPosition, float InTime, FVector& OutNormal) const override;
	virtual float GetWaveAttenuationFactor(
		const FVector& InPosition, float InWaterDepth) const override;
};
